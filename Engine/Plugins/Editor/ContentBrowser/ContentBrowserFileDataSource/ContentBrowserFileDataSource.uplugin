{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Content Browser - File Data Source",
	"Description" : "Data Source plugin providing loose file support for the Content Browser",
	"Category" : "Content Browser",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"SupportedPrograms": ["LiveLinkHub"],
	"Modules" :
	[
		{
			"Name" : "ContentBrowserFileDataSource",
			"Type" : "EditorAndProgram",
			"LoadingPhase" : "Default"
		}
	]
}