// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ContentBrowserFileDataSource : ModuleRules
	{
		public ContentBrowserFileDataSource(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "CBFileDataSource";

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"ContentBrowserData",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetTools",
					"Slate",
					"SlateCore",
					"SourceControl",
					"SourceControlWindows",
					"ToolMenus",
					"UnrealEd",
					"UncontrolledChangelists",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"DirectoryWatcher",
				}
			);
			
			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					"DirectoryWatcher",
				}
			);
		}
	}
}
