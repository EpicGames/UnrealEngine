// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserFileDataCore.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserFileDataPayload.h"
#include "Misc/FileHelper.h"
#include "IAssetTools.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/Paths.h"
#include "AssetThumbnail.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "SourceControlHelpers.h"

#define LOCTEXT_NAMESPACE "ContentBrowserFileDataSource"

namespace ContentBrowserFileData
{

FContentBrowserItemData CreateFolderItem(
	UContentBrowserDataSource* InOwnerDataSource,
	const FName InVirtualPath,
	const FName InInternalPath,
	const FString& InFilename,
	TWeakPtr<const ContentBrowserFileData::FDirectoryActions> InDirectoryActions)
{
	const FString FolderItemName = FPaths::GetCleanFilename(InFilename);
	return FContentBrowserItemData(
		InOwnerDataSource,
		EContentBrowserItemFlags::Type_Folder | EContentBrowserItemFlags::Category_Misc,
		InVirtualPath,
		*FolderItemName,
		FText(),
		MakeShared<FContentBrowserFolderItemDataPayload>(InInternalPath, InFilename, MoveTemp(InDirectoryActions)),
		{ InInternalPath });
}

FContentBrowserItemData CreateFileItem(
	UContentBrowserDataSource* InOwnerDataSource,
	const FName InVirtualPath,
	const FName InInternalPath,
	const FString& InFilename,
	TWeakPtr<const ContentBrowserFileData::FFileActions> InFileActions)
{
	const FString FileItemName = FPaths::GetBaseFilename(InFilename);
	return FContentBrowserItemData(
		InOwnerDataSource,
		EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Category_Misc,
		InVirtualPath,
		*FileItemName,
		FText(),
		MakeShared<FContentBrowserFileItemDataPayload>(InInternalPath, InFilename, MoveTemp(InFileActions)),
		{ InInternalPath });
}

TSharedPtr<const FContentBrowserFolderItemDataPayload> GetFolderItemPayload(
	const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem)
{
	if (InItem.GetOwnerDataSource() == InOwnerDataSource && InItem.IsFolder())
	{
		return StaticCastSharedPtr<const FContentBrowserFolderItemDataPayload>(InItem.GetPayload());
	}
	return nullptr;
}

TSharedPtr<const FContentBrowserFileItemDataPayload> GetFileItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem)
{
	if (InItem.GetOwnerDataSource() == InOwnerDataSource && InItem.IsFile())
	{
		return StaticCastSharedPtr<const FContentBrowserFileItemDataPayload>(InItem.GetPayload());
	}
	return nullptr;
}

void EnumerateFolderItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserFolderItemDataPayload>&)> InFolderPayloadCallback)
{
	for (const FContentBrowserItemData& Item : InItems)
	{
		if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InOwnerDataSource, Item))
		{
			if (!InFolderPayloadCallback(FolderPayload.ToSharedRef()))
			{
				break;
			}
		}
	}
}

void EnumerateFileItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserFileItemDataPayload>&)> InFilePayloadCallback)
{
	for (const FContentBrowserItemData& Item : InItems)
	{
		if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, Item))
		{
			if (!InFilePayloadCallback(FilePayload.ToSharedRef()))
			{
				break;
			}
		}
	}
}

void EnumerateItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserFolderItemDataPayload>&)> InFolderPayloadCallback, TFunctionRef<bool(const TSharedRef<const FContentBrowserFileItemDataPayload>&)> InFilePayloadCallback)
{
	for (const FContentBrowserItemData& Item : InItems)
	{
		if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InOwnerDataSource, Item))
		{
			if (!InFolderPayloadCallback(FolderPayload.ToSharedRef()))
			{
				break;
			}
		}

		if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, Item))
		{
			if (!InFilePayloadCallback(FilePayload.ToSharedRef()))
			{
				break;
			}
		}
	}
}

void SetOptionalErrorMessage(FText* OutErrorMsg, FText InErrorMsg)
{
	if (OutErrorMsg)
	{
		*OutErrorMsg = MoveTemp(InErrorMsg);
	}
}

void MakeUniqueFilename(FString& InOutFilename)
{
	const FString Extension = FPaths::GetExtension(InOutFilename, /*bIncludeDot*/true);
	const FString FilePath = FPaths::GetPath(InOutFilename);
	const FString BaseFileName = FPaths::GetBaseFilename(InOutFilename);

	int32 IntSuffix = 0;
	FString TrailingInteger;
	FString TrimmedBaseFileName = BaseFileName;
	{
		int32 CharIndex = BaseFileName.Len() - 1;
		while (CharIndex >= 0 && BaseFileName[CharIndex] >= TEXT('0') && BaseFileName[CharIndex] <= TEXT('9'))
		{
			--CharIndex;
		}
		if (BaseFileName.Len() > 0 && CharIndex == -1)
		{
			// This is the all numeric name, in this case we'd like to append _number, because just adding a number isn't great
			TrimmedBaseFileName += TEXT("_");
			IntSuffix = 2;
		}
		if (CharIndex >= 0 && CharIndex < BaseFileName.Len() - 1)
		{
			TrailingInteger = BaseFileName.RightChop(CharIndex + 1);
			TrimmedBaseFileName = BaseFileName.Left(CharIndex + 1);
			IntSuffix = FCString::Atoi(*TrailingInteger);
		}
	}

	bool bFileExists = false;
	do
	{
		bFileExists = false;

		FString PotentialBaseFileName;
		if (IntSuffix < 1)
		{
			PotentialBaseFileName = BaseFileName;
		}
		else
		{
			FString Suffix = FString::Printf(TEXT("%d"), IntSuffix);
			while (Suffix.Len() < TrailingInteger.Len())
			{
				Suffix.InsertAt(0, TEXT("0"));
			}
			PotentialBaseFileName = FString::Printf(TEXT("%s%s"), *TrimmedBaseFileName, *Suffix);
		}

		InOutFilename = FilePath / PotentialBaseFileName + Extension;
		bFileExists = IFileManager::Get().FileExists(*InOutFilename);

		IntSuffix++;
	}
	while (bFileExists);
}

bool DeleteDirectoryIfEmpty(const FString& InDirToDelete)
{
	IFileManager& FileManager = IFileManager::Get();

	bool bDirContainsFiles = false;
	FileManager.IterateDirectoryRecursively(*InDirToDelete, [&bDirContainsFiles](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
	{
		if (!bIsDirectory)
		{
			bDirContainsFiles = true;
		}

		return !bDirContainsFiles;
	});

	if (!bDirContainsFiles)
	{
		return FileManager.DeleteDirectory(*InDirToDelete, false, true);
	}

	return false;
}

struct FDirectoryToMigrate
{
	FString DestDir;
	FString SourceDir;
};

bool MigrateDirectoryContents(const FFileConfigData& InConfig, TArrayView<const FDirectoryToMigrate> InDirectoriesToMigrate, const bool bDeleteSource)
{
	bool bDidMigrate = false;

	FScopedSlowTask SlowTask(InDirectoriesToMigrate.Num(), LOCTEXT("MigrateDirectoryContents", "Migrating Directory Contents..."));
	SlowTask.MakeDialogDelayed(1.0f, /*bShowCancel*/true);

	// Copy the relevant files from each source to the destination, preserving the directory structure within the source
	int32 TaskProgress = 0;
	for (const FDirectoryToMigrate& DirectoryToMigrate : InDirectoriesToMigrate)
	{
		if (SlowTask.ShouldCancel())
		{
			break;
		}

		IFileManager& FileManager = IFileManager::Get();

		// Ensure the destination directory exists
		if (FileManager.MakeDirectory(*DirectoryToMigrate.DestDir, true))
		{
			TArray<FString> FilesToAdd;
			TArray<FString> FilesToDelete;
			FileManager.IterateDirectoryRecursively(*DirectoryToMigrate.SourceDir, [&FilesToAdd, &FilesToDelete, &InConfig, &DirectoryToMigrate, &SlowTask, &FileManager, &bDidMigrate](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
			{
				if (bIsDirectory)
				{
					FString NewDestDir = FilenameOrDirectory;
					NewDestDir.ReplaceInline(*DirectoryToMigrate.SourceDir, *DirectoryToMigrate.DestDir);

					bDidMigrate |= FileManager.MakeDirectory(*NewDestDir, false);
				}
				else
				{
					const bool bValidFile = InConfig.FindFileActionsForFilename(FilenameOrDirectory).IsValid();
					if (bValidFile)
					{
						FString NewDestFile = FilenameOrDirectory;
						NewDestFile.ReplaceInline(*DirectoryToMigrate.SourceDir, *DirectoryToMigrate.DestDir);

						if (FileManager.Copy(*NewDestFile, FilenameOrDirectory, /*bReplace*/false) == COPY_OK)
						{
							bDidMigrate = true;;
							FilesToAdd.Add(NewDestFile);
							FilesToDelete.Add(FilenameOrDirectory);
						}
					}
				}

				SlowTask.EnterProgressFrame(0.0f);
				return !SlowTask.ShouldCancel();
			});

			USourceControlHelpers::MarkFilesForAdd(FilesToAdd, /*bSilent*/true);
			if (bDeleteSource)
			{
				if (!USourceControlHelpers::MarkFilesForDelete(FilesToDelete, /*bSilent*/true))
				{
					for (const FString& FileToDelete : FilesToDelete)
					{
						FileManager.Delete(*FileToDelete);
					}
				}

				// Remove the source if empty of *all* files
				DeleteDirectoryIfEmpty(DirectoryToMigrate.SourceDir);
			}
		}

		SlowTask.EnterProgressFrame(1.0f);
	}

	return bDidMigrate;
}

bool CanModifyDirectory(const FString& InFilename, FText* OutErrorMsg)
{
	if (IFileManager::Get().IsReadOnly(*InFilename))
	{
		SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_DirectoryReadOnly", "Directory is read-only"));
		return false;
	}

	return true;
}

bool CanModifyFile(const FString& InFilename, FText* OutErrorMsg)
{
	if (IFileManager::Get().IsReadOnly(*InFilename))
	{
		SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_FileReadOnly", "File is read-only"));
		return false;
	}

	return true;
}

bool CanModifyItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return CanModifyFolderItem(*FolderPayload, OutErrorMsg);
	}

	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanModifyFileItem(*FilePayload, OutErrorMsg);
	}

	return false;
}

bool CanModifyFolderItem(const FContentBrowserFolderItemDataPayload& InFolderPayload, FText* OutErrorMsg)
{
	return CanModifyDirectory(InFolderPayload.GetFilename(), OutErrorMsg);
}

bool CanModifyFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, FText* OutErrorMsg)
{
	return CanModifyFile(InFilePayload.GetFilename(), OutErrorMsg);
}

bool CanEditItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanEditFileItem(*FilePayload, OutErrorMsg);
	}

	return false;
}

bool CanEditFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, FText* OutErrorMsg)
{
	if (TSharedPtr<const FFileActions> FileActions = InFilePayload.GetFileActions())
	{
		return !FileActions->CanEdit.IsBound() || FileActions->CanEdit.Execute(InFilePayload.GetInternalPath(), InFilePayload.GetFilename(), OutErrorMsg);
	}

	return false;
}

bool EditItems(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems)
{
	TArray<TSharedRef<const FContentBrowserFileItemDataPayload>, TInlineAllocator<16>> FilePayloads;

	EnumerateFileItemPayloads(InOwnerDataSource, InItems, [&FilePayloads](const TSharedRef<const FContentBrowserFileItemDataPayload>& InFilePayload)
	{
		if (CanEditFileItem(*InFilePayload, nullptr))
		{
			FilePayloads.Add(InFilePayload);
		}
		return true;
	});

	return EditFileItems(FilePayloads);
}

bool EditFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads)
{
	bool bDidEdit = false;

	for (const TSharedRef<const FContentBrowserFileItemDataPayload>& FilePayload : InFilePayloads)
	{
		if (TSharedPtr<const FFileActions> FileActions = FilePayload->GetFileActions())
		{
			if (FileActions->Edit.IsBound())
			{
				bDidEdit |= FileActions->Edit.Execute(FilePayload->GetInternalPath(), FilePayload->GetFilename());
			}
			else
			{
				FPlatformProcess::LaunchFileInDefaultExternalApplication(*FilePayload->GetFilename(), nullptr, FileActions->DefaultEditVerb);
				bDidEdit = true;
			}
		}
	}

	return bDidEdit;
}

bool CanPreviewItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanPreviewFileItem(*FilePayload, OutErrorMsg);
	}

	return false;
}

bool CanPreviewFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, FText* OutErrorMsg)
{
	if (TSharedPtr<const FFileActions> FileActions = InFilePayload.GetFileActions())
	{
		return (!FileActions->CanPreview.IsBound() || FileActions->CanPreview.Execute(InFilePayload.GetInternalPath(), InFilePayload.GetFilename(), OutErrorMsg))
			&& FileActions->Preview.IsBound();
	}

	return false;
}

bool PreviewItems(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems)
{
	TArray<TSharedRef<const FContentBrowserFileItemDataPayload>, TInlineAllocator<16>> FilePayloads;

	EnumerateFileItemPayloads(InOwnerDataSource, InItems, [&FilePayloads](const TSharedRef<const FContentBrowserFileItemDataPayload>& InFilePayload)
	{
		if (CanPreviewFileItem(*InFilePayload, nullptr))
		{
			FilePayloads.Add(InFilePayload);
		}
		return true;
	});

	return PreviewFileItems(FilePayloads);
}

bool PreviewFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads)
{
	bool bDidPreview = false;

	for (const TSharedRef<const FContentBrowserFileItemDataPayload>& FilePayload : InFilePayloads)
	{
		if (TSharedPtr<const FFileActions> FileActions = FilePayload->GetFileActions())
		{
			if (FileActions->Preview.IsBound())
			{
				bDidPreview |= FileActions->Preview.Execute(FilePayload->GetInternalPath(), FilePayload->GetFilename());
			}
		}
	}

	return bDidPreview;
}

bool CanDuplicateItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanDuplicateFileItem(*FilePayload, OutErrorMsg);
	}

	return false;
}

bool CanDuplicateFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, FText* OutErrorMsg)
{
	if (TSharedPtr<const FFileActions> FileActions = InFilePayload.GetFileActions())
	{
		if (FileActions->CanDuplicate.IsBound())
		{
			return FileActions->CanDuplicate.Execute(InFilePayload.GetInternalPath(), InFilePayload.GetFilename(), OutErrorMsg);
		}
	}

	return true;
}

bool DuplicateItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, TSharedPtr<const FContentBrowserFileItemDataPayload_Duplication>& OutNewItemPayload)
{
	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return DuplicateFileItem(*FilePayload, OutNewItemPayload);
	}

	return false;
}

bool DuplicateFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, TSharedPtr<const FContentBrowserFileItemDataPayload_Duplication>& OutNewItemPayload)
{
	FString NewFilename = InFilePayload.GetFilename();
	MakeUniqueFilename(NewFilename);

	const FString NewInternalPath = FPaths::GetPath(InFilePayload.GetInternalPath().ToString()) / FPaths::GetCleanFilename(NewFilename);

	OutNewItemPayload = MakeShared<FContentBrowserFileItemDataPayload_Duplication>(*NewInternalPath, NewFilename, InFilePayload.GetFileActions(), InFilePayload.GetFilename());
	
	return true;
}

bool DuplicateItems(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TArray<TSharedRef<const FContentBrowserFileItemDataPayload>>& OutNewFilePayloads)
{
	TArray<TSharedRef<const FContentBrowserFileItemDataPayload>, TInlineAllocator<16>> FilePayloads;

	EnumerateFileItemPayloads(InOwnerDataSource, InItems, [&FilePayloads](const TSharedRef<const FContentBrowserFileItemDataPayload>& InFilePayload)
	{
		if (CanDuplicateFileItem(*InFilePayload, nullptr))
		{
			FilePayloads.Add(InFilePayload);
		}
		return true;
	});

	return DuplicateFileItems(FilePayloads, OutNewFilePayloads);
}

bool DuplicateFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads, TArray<TSharedRef<const FContentBrowserFileItemDataPayload>>& OutNewFilePayloads)
{
	bool bDidDuplicate = false;
	TArray<FString> FilesToAdd;

	for (const TSharedRef<const FContentBrowserFileItemDataPayload>& FilePayload : InFilePayloads)
	{
		FString NewFilename = FilePayload->GetFilename();
		MakeUniqueFilename(NewFilename);

		if (IFileManager::Get().Copy(*NewFilename, *FilePayload->GetFilename(), /*bReplace*/false) == COPY_OK)
		{
			const FString NewInternalPath = FPaths::GetPath(FilePayload->GetInternalPath().ToString()) / FPaths::GetCleanFilename(NewFilename);
			OutNewFilePayloads.Emplace(MakeShared<FContentBrowserFileItemDataPayload>(*NewInternalPath, NewFilename, FilePayload->GetFileActions()));
			bDidDuplicate = true;
			FilesToAdd.Add(NewFilename);
		}
	}

	USourceControlHelpers::MarkFilesForAdd(FilesToAdd, /*bSilent*/true);

	return bDidDuplicate;
}

bool CanDeleteItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return CanDeleteFolderItem(*FolderPayload, OutErrorMsg);
	}

	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanDeleteFileItem(*FilePayload, OutErrorMsg);
	}

	return false;
}

bool CanDeleteFolderItem(const FContentBrowserFolderItemDataPayload& InFolderPayload, FText* OutErrorMsg)
{
	if (TSharedPtr<const FDirectoryActions> DirectoryActions = InFolderPayload.GetDirectoryActions())
	{
		if (DirectoryActions->CanDelete.IsBound())
		{
			return DirectoryActions->CanDelete.Execute(InFolderPayload.GetInternalPath(), InFolderPayload.GetFilename(), OutErrorMsg);
		}
	}

	return CanModifyFolderItem(InFolderPayload, OutErrorMsg);
}

bool CanDeleteFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, FText* OutErrorMsg)
{
	if (TSharedPtr<const FFileActions> FileActions = InFilePayload.GetFileActions())
	{
		if (FileActions->CanDelete.IsBound())
		{
			return FileActions->CanDelete.Execute(InFilePayload.GetInternalPath(), InFilePayload.GetFilename(), OutErrorMsg);
		}
	}

	return CanModifyFileItem(InFilePayload, OutErrorMsg);
}

bool DeleteItems(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems)
{
	TArray<TSharedRef<const FContentBrowserFolderItemDataPayload>, TInlineAllocator<16>> FolderPayloads;
	TArray<TSharedRef<const FContentBrowserFileItemDataPayload>, TInlineAllocator<16>> FilePayloads;

	auto ProcessFolderItem = [&FolderPayloads](const TSharedRef<const FContentBrowserFolderItemDataPayload>& InFolderPayload)
	{
		if (CanDeleteFolderItem(*InFolderPayload, nullptr))
		{
			FolderPayloads.Add(InFolderPayload);
		}
		return true;
	};

	auto ProcessFileItem = [&FilePayloads](const TSharedRef<const FContentBrowserFileItemDataPayload>& InFilePayload)
	{
		if (CanDeleteFileItem(*InFilePayload, nullptr))
		{
			FilePayloads.Add(InFilePayload);
		}
		return true;
	};

	EnumerateItemPayloads(InOwnerDataSource, InItems, ProcessFolderItem, ProcessFileItem);

	bool bDidDelete = false;

	if (FolderPayloads.Num() > 0)
	{
		bDidDelete |= DeleteFolderItems(FolderPayloads);
	}

	if (FilePayloads.Num() > 0)
	{
		bDidDelete |= DeleteFileItems(FilePayloads);
	}

	return bDidDelete;
}

bool DeleteFolderItems(TArrayView<const TSharedRef<const FContentBrowserFolderItemDataPayload>> InFolderPayloads)
{
	bool bDidDelete = false;

	for (const TSharedRef<const FContentBrowserFolderItemDataPayload>& FolderPayload : InFolderPayloads)
	{
		// TODO: This will only delete the folder if it is empty - we should have a way to delete any known file types with a UI too
		bDidDelete |= DeleteDirectoryIfEmpty(FolderPayload->GetFilename());
	}

	return bDidDelete;
}

bool DeleteFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads)
{
	bool bDidDelete = false;
	TArray<FString> FilesToDelete;

	for (const TSharedRef<const FContentBrowserFileItemDataPayload>& FilePayload : InFilePayloads)
	{
		FilesToDelete.Add(FilePayload->GetFilename());
	}

	if (USourceControlHelpers::MarkFilesForDelete(FilesToDelete, /*bSilent*/true))
	{
		bDidDelete = true;
	}
	else
	{
		IFileManager& FileManager = IFileManager::Get();
		for (const FString& FileToDelete : FilesToDelete)
		{
			bDidDelete |= FileManager.Delete(*FileToDelete);
		}
	}

	return bDidDelete;
}

bool CanRenameItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const FString* InNewName, FText* OutErrorMsg)
{
	constexpr bool bCheckUniqueName = true;
	return CanRenameItem(InOwnerDataSource, InItem, bCheckUniqueName, InNewName, OutErrorMsg);
}

bool CanRenameItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool bCheckUniqueName, const FString* InNewName, FText* OutErrorMsg)
{
	if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return CanRenameFolderItem(*FolderPayload, bCheckUniqueName, InNewName, OutErrorMsg);
	}

	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return CanRenameFileItem(*FilePayload, bCheckUniqueName, InNewName, OutErrorMsg);
	}

	return false;
}

bool IsValidName(const bool bIsDirectory, const bool bCheckUniqueName, const FString& InCurrentFilename, const FString& InNewName, FText* OutErrorMsg)
{
	// Make sure the name is not already a class or otherwise invalid for saving
	FText LocalErrorMsg;
	if (!FFileHelper::IsFilenameValidForSaving(InNewName, LocalErrorMsg))
	{
		SetOptionalErrorMessage(OutErrorMsg, LocalErrorMsg);

		// Return false to indicate that the user should enter a new name
		return false;
	}

	// Make sure the new name only contains valid characters
	if (!FName::IsValidXName(InNewName, INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, OutErrorMsg))
	{
		// Return false to indicate that the user should enter a new name
		return false;
	}

	// Check custom filter set by external module
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	if (!AssetToolsModule.Get().IsNameAllowed(InNewName, OutErrorMsg))
	{
		return false;
	}

	// Is this name unique?
	if (bCheckUniqueName)
	{
		const FString NewFilename = FPaths::GetPath(InCurrentFilename) / InNewName + FPaths::GetExtension(InCurrentFilename, /*bIncludeDot*/true);
		if (bIsDirectory)
		{
			if (IFileManager::Get().DirectoryExists(*NewFilename))
			{
				SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_DirectoryExists", "A directory already exists with this name"));
				return false;
			}
		}
		else
		{
			if (IFileManager::Get().FileExists(*NewFilename))
			{
				SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_FileExists", "A file already exists with this name"));
				return false;
			}
		}
	}

	return true;
}

bool CanRenameFolderItem(const FContentBrowserFolderItemDataPayload& InFolderPayload, const bool bCheckUniqueName, const FString* InNewName, FText* OutErrorMsg)
{
	if (TSharedPtr<const FDirectoryActions> DirectoryActions = InFolderPayload.GetDirectoryActions())
	{
		if (DirectoryActions->CanRename.IsBound())
		{
			return DirectoryActions->CanRename.Execute(InFolderPayload.GetInternalPath(), InFolderPayload.GetFilename(), InNewName, OutErrorMsg);
		}
	}

	if (InNewName && !IsValidName(/*bIsDirectory*/true, bCheckUniqueName, InFolderPayload.GetFilename(), *InNewName, OutErrorMsg))
	{
		return false;
	}

	return CanModifyFolderItem(InFolderPayload, OutErrorMsg);
}

bool CanRenameFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, const bool bCheckUniqueName, const FString* InNewName, FText* OutErrorMsg)
{
	if (TSharedPtr<const FFileActions> FileActions = InFilePayload.GetFileActions())
	{
		if (FileActions->CanRename.IsBound())
		{
			return FileActions->CanRename.Execute(InFilePayload.GetInternalPath(), InFilePayload.GetFilename(), InNewName, OutErrorMsg);
		}
	}

	if (InNewName && !IsValidName(/*bIsDirectory*/false, bCheckUniqueName, InFilePayload.GetFilename(), *InNewName, OutErrorMsg))
	{
		return false;
	}

	return CanModifyFileItem(InFilePayload, OutErrorMsg);
}

bool RenameItem(const FFileConfigData& InConfig, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const FString& InNewName, FName& OutNewInternalPath, FString& OutNewFilename)
{
	if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InOwnerDataSource, InItem))
	{
		if (CanRenameFolderItem(*FolderPayload, /*bCheckUniqueName*/false, &InNewName, nullptr))
		{
			return RenameFolderItem(InConfig, *FolderPayload, InNewName, OutNewInternalPath, OutNewFilename);
		}
	}

	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		if (CanRenameFileItem(*FilePayload, /*bCheckUniqueName*/false, &InNewName, nullptr))
		{
			return RenameFileItem(*FilePayload, InNewName, OutNewInternalPath, OutNewFilename);
		}
	}

	return false;
}

bool RenameFolderItem(const FFileConfigData& InConfig, const FContentBrowserFolderItemDataPayload& InFolderPayload, const FString& InNewName, FName& OutNewInternalPath, FString& OutNewFilename)
{
	const FString NewFilename = FPaths::GetPath(InFolderPayload.GetFilename()) / InNewName;
	const FDirectoryToMigrate DirectoryToMigrate{ NewFilename, InFolderPayload.GetFilename() };
	if (MigrateDirectoryContents(InConfig, MakeArrayView(&DirectoryToMigrate, 1), /*bDeleteSource*/true))
	{
		OutNewInternalPath = *(FPaths::GetPath(InFolderPayload.GetInternalPath().ToString()) / InNewName);
		OutNewFilename = NewFilename;
		return true;
	}

	return false;
}

bool RenameFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, const FString& InNewName, FName& OutNewInternalPath, FString& OutNewFilename)
{
	IFileManager& FileManager = IFileManager::Get();

	const FString Extension = FPaths::GetExtension(InFilePayload.GetFilename(), /*bIncludeDot*/true);
	const FString NewFilename = FPaths::GetPath(InFilePayload.GetFilename()) / InNewName + Extension;
	if (FileManager.Copy(*NewFilename, *InFilePayload.GetFilename(), /*bReplace*/false) == COPY_OK)
	{
		// TODO: This could be a branch (rather than delete + add)
		if (!USourceControlHelpers::MarkFileForDelete(InFilePayload.GetFilename(), /*bSilent*/false))
		{
			FileManager.Delete(*InFilePayload.GetFilename());
		}
		USourceControlHelpers::MarkFileForAdd(NewFilename, /*bSilent*/true);

		OutNewInternalPath = *(FPaths::GetPath(InFilePayload.GetInternalPath().ToString()) / InNewName + Extension);
		OutNewFilename = NewFilename;
		return true;
	}

	return false;
}

bool CopyItems(const FFileConfigData& InConfig, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const FString& InDestDiskPath)
{
	// The destination path must be writable
	if (!CanModifyDirectory(InDestDiskPath, nullptr))
	{
		return false;
	}

	TArray<TSharedRef<const FContentBrowserFolderItemDataPayload>, TInlineAllocator<16>> FolderPayloads;
	TArray<TSharedRef<const FContentBrowserFileItemDataPayload>, TInlineAllocator<16>> FilePayloads;

	auto ProcessFolderItem = [&FolderPayloads](const TSharedRef<const FContentBrowserFolderItemDataPayload>& InFolderPayload)
	{
		FolderPayloads.Add(InFolderPayload);
		return true;
	};

	auto ProcessFileItem = [&FilePayloads](const TSharedRef<const FContentBrowserFileItemDataPayload>& InFilePayload)
	{
		FilePayloads.Add(InFilePayload);
		return true;
	};

	EnumerateItemPayloads(InOwnerDataSource, InItems, ProcessFolderItem, ProcessFileItem);

	bool bDidCopy = false;

	if (FolderPayloads.Num() > 0)
	{
		bDidCopy |= CopyFolderItems(InConfig, FolderPayloads, InDestDiskPath);
	}

	if (FilePayloads.Num() > 0)
	{
		bDidCopy |= CopyFileItems(FilePayloads, InDestDiskPath);
	}

	return bDidCopy;
}

bool CopyFolderItems(const FFileConfigData& InConfig, TArrayView<const TSharedRef<const FContentBrowserFolderItemDataPayload>> InFolderPayloads, const FString& InDestDiskPath)
{
	TArray<FDirectoryToMigrate, TInlineAllocator<16>> DirectoriesToMigrate;
	for (const TSharedRef<const FContentBrowserFolderItemDataPayload>& FolderPayload : InFolderPayloads)
	{
		DirectoriesToMigrate.Add(FDirectoryToMigrate{ InDestDiskPath / FPaths::GetCleanFilename(FolderPayload->GetFilename()), FolderPayload->GetFilename() });
	}

	return MigrateDirectoryContents(InConfig, DirectoriesToMigrate, /*bDeleteSource*/false);
}

bool CopyFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads, const FString& InDestDiskPath)
{
	bool bDidCopy = false;
	TArray<FString> FilesToAdd;

	for (const TSharedRef<const FContentBrowserFileItemDataPayload>& FilePayload : InFilePayloads)
	{
		const FString NewFilename = InDestDiskPath / FPaths::GetCleanFilename(FilePayload->GetFilename());
		if (IFileManager::Get().Copy(*NewFilename, *FilePayload->GetFilename(), /*bReplace*/false) == COPY_OK)
		{
			bDidCopy = true;
			FilesToAdd.Add(NewFilename);
		}
	}

	USourceControlHelpers::MarkFilesForAdd(FilesToAdd, /*bSilent*/true);

	return bDidCopy;
}

bool MoveItems(const FFileConfigData& InConfig, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const FString& InDestDiskPath)
{
	// The destination path must be writable
	if (!CanModifyDirectory(InDestDiskPath, nullptr))
	{
		return false;
	}

	TArray<TSharedRef<const FContentBrowserFolderItemDataPayload>, TInlineAllocator<16>> FolderPayloads;
	TArray<TSharedRef<const FContentBrowserFileItemDataPayload>, TInlineAllocator<16>> FilePayloads;

	auto ProcessFolderItem = [&FolderPayloads](const TSharedRef<const FContentBrowserFolderItemDataPayload>& InFolderPayload)
	{
		// Moving has to be able to delete the original item
		if (CanModifyFolderItem(*InFolderPayload, nullptr))
		{
			FolderPayloads.Add(InFolderPayload);
		}
		return true;
	};

	auto ProcessFileItem = [&FilePayloads](const TSharedRef<const FContentBrowserFileItemDataPayload>& InFilePayload)
	{
		// Moving has to be able to delete the original item
		if (CanModifyFileItem(*InFilePayload, nullptr))
		{
			FilePayloads.Add(InFilePayload);
		}
		return true;
	};

	EnumerateItemPayloads(InOwnerDataSource, InItems, ProcessFolderItem, ProcessFileItem);

	bool bDidMove = false;

	if (FolderPayloads.Num() > 0)
	{
		bDidMove |= MoveFolderItems(InConfig, FolderPayloads, InDestDiskPath);
	}

	if (FilePayloads.Num() > 0)
	{
		bDidMove |= MoveFileItems(FilePayloads, InDestDiskPath);
	}

	return bDidMove;
}

bool MoveFolderItems(const FFileConfigData& InConfig, TArrayView<const TSharedRef<const FContentBrowserFolderItemDataPayload>> InFolderPayloads, const FString& InDestDiskPath)
{
	TArray<FDirectoryToMigrate, TInlineAllocator<16>> DirectoriesToMigrate;
	for (const TSharedRef<const FContentBrowserFolderItemDataPayload>& FolderPayload : InFolderPayloads)
	{
		DirectoriesToMigrate.Add(FDirectoryToMigrate{ InDestDiskPath / FPaths::GetCleanFilename(FolderPayload->GetFilename()), FolderPayload->GetFilename() });
	}

	return MigrateDirectoryContents(InConfig, DirectoriesToMigrate, /*bDeleteSource*/true);
}

bool MoveFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads, const FString& InDestDiskPath)
{
	bool bDidMove = false;
	TArray<FString> FilesToAdd;
	TArray<FString> FilesToDelete;

	IFileManager& FileManager = IFileManager::Get();

	for (const TSharedRef<const FContentBrowserFileItemDataPayload>& FilePayload : InFilePayloads)
	{
		// TODO: This could be a branch (rather than delete + add)
		const FString NewFilename = InDestDiskPath / FPaths::GetCleanFilename(FilePayload->GetFilename());
		if (FileManager.Copy(*NewFilename, *FilePayload->GetFilename(), /*bReplace*/false) == COPY_OK)
		{
			bDidMove = true;
			FilesToAdd.Add(NewFilename);
			FilesToDelete.Add(FilePayload->GetFilename());
		}
	}

	USourceControlHelpers::MarkFilesForAdd(FilesToAdd, /*bSilent*/true);
	if (!USourceControlHelpers::MarkFilesForDelete(FilesToDelete, /*bSilent*/true))
	{
		for (const FString& FileToDelete : FilesToDelete)
		{
			FileManager.Delete(*FileToDelete);
		}
	}

	return bDidMove;
}

bool UpdateItemThumbnail(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail)
{
	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return UpdateFileItemThumbnail(*FilePayload, InThumbnail);
	}

	return false;
}

bool UpdateFileItemThumbnail(const FContentBrowserFileItemDataPayload& InFilePayload, FAssetThumbnail& InThumbnail)
{
	if (TSharedPtr<const FFileActions> FileActions = InFilePayload.GetFileActions())
	{
		InThumbnail.SetAsset(FAssetData(InFilePayload.GetInternalPath(), *FPaths::GetPath(InFilePayload.GetInternalPath().ToString()), *FPaths::GetBaseFilename(InFilePayload.GetFilename()), FileActions->TypeName));
		return true;
	}

	return false;
}

bool GetItemPhysicalPath(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& OutDiskPath)
{
	if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return GetFolderItemPhysicalPath(*FolderPayload, OutDiskPath);
	}

	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetFileItemPhysicalPath(*FilePayload, OutDiskPath);
	}

	return false;
}

bool GetFolderItemPhysicalPath(const FContentBrowserFolderItemDataPayload& InFolderPayload, FString& OutDiskPath)
{
	const FString& Filename = InFolderPayload.GetFilename();
	if (!Filename.IsEmpty())
	{
		OutDiskPath = Filename;
		return true;
	}

	return false;
}

bool GetFileItemPhysicalPath(const FContentBrowserFileItemDataPayload& InFilePayload, FString& OutDiskPath)
{
	const FString& Filename = InFilePayload.GetFilename();
	if (!Filename.IsEmpty())
	{
		OutDiskPath = Filename;
		return true;
	}

	return false;
}

bool GetItemAttribute(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return GetFolderItemAttribute(*FolderPayload, InIncludeMetaData, InAttributeKey, OutAttributeValue);
	}

	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetFileItemAttribute(*FilePayload, InIncludeMetaData, InAttributeKey, OutAttributeValue);
	}

	return false;
}

bool GetFolderItemAttribute(const FContentBrowserFolderItemDataPayload& InFolderPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	TSharedPtr<const FDirectoryActions> DirectoryActions = InFolderPayload.GetDirectoryActions();
	if (!DirectoryActions)
	{
		return false;
	}

	// External attributes take priority
	if (DirectoryActions->GetAttribute.IsBound() && DirectoryActions->GetAttribute.Execute(InFolderPayload.GetInternalPath(), InFolderPayload.GetFilename(), InIncludeMetaData, InAttributeKey, OutAttributeValue))
	{
		return true;
	}

	return false;
}

bool GetFileItemAttribute(const FContentBrowserFileItemDataPayload& InFilePayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	TSharedPtr<const FFileActions> FileActions = InFilePayload.GetFileActions();
	if (!FileActions)
	{
		return false;
	}

	// External attributes take priority
	if (FileActions->GetAttribute.IsBound() && FileActions->GetAttribute.Execute(InFilePayload.GetInternalPath(), InFilePayload.GetFilename(), InIncludeMetaData, InAttributeKey, OutAttributeValue))
	{
		return true;
	}

	// Hard-coded attribute keys
	{
		static const FName NAME_Type = "Type";
	
		if (InAttributeKey == ContentBrowserItemAttributes::ItemTypeName || InAttributeKey == NAME_Class || InAttributeKey == NAME_Type)
		{
			OutAttributeValue.SetValue(FileActions->TypeName.ToString());
			return true;
		}
	
		if (InAttributeKey == ContentBrowserItemAttributes::ItemTypeDisplayName)
		{
			OutAttributeValue.SetValue(FileActions->TypeDisplayName);
			return true;
		}
	
		if (InAttributeKey == ContentBrowserItemAttributes::ItemColor)
		{
			OutAttributeValue.SetValue(FileActions->TypeColor.ToString());
			return true;
		}
	}

	return false;
}

bool GetItemAttributes(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InOwnerDataSource, InItem))
	{
		return GetFolderItemAttributes(*FolderPayload, InIncludeMetaData, OutAttributeValues);
	}

	if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InOwnerDataSource, InItem))
	{
		return GetFileItemAttributes(*FilePayload, InIncludeMetaData, OutAttributeValues);
	}

	return false;
}

bool GetFolderItemAttributes(const FContentBrowserFolderItemDataPayload& InFolderPayload, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	TSharedPtr<const FDirectoryActions> DirectoryActions = InFolderPayload.GetDirectoryActions();
	if (!DirectoryActions)
	{
		return false;
	}

	// External attributes take priority
	if (DirectoryActions->GetAttributes.IsBound() && DirectoryActions->GetAttributes.Execute(InFolderPayload.GetInternalPath(), InFolderPayload.GetFilename(), InIncludeMetaData, OutAttributeValues))
	{
		return true;
	}

	return false;
}

bool GetFileItemAttributes(const FContentBrowserFileItemDataPayload& InFilePayload, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	TSharedPtr<const FFileActions> FileActions = InFilePayload.GetFileActions();
	if (!FileActions)
	{
		return false;
	}

	// External attributes take priority
	if (FileActions->GetAttributes.IsBound() && FileActions->GetAttributes.Execute(InFilePayload.GetInternalPath(), InFilePayload.GetFilename(), InIncludeMetaData, OutAttributeValues))
	{
		return true;
	}

	return false;
}

bool FDefaultFileActions::ItemPassesFilter(const FName InFilePath, const FString& InFilename, const FContentBrowserDataFilter& InFilter, const bool bIsFile)
{
	const FContentBrowserDataPackageFilter* PackageFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataPackageFilter>();
	if (PackageFilter && PackageFilter->PathPermissionList && PackageFilter->PathPermissionList->HasFiltering())
	{
		return PackageFilter->PathPermissionList->PassesStartsWithFilter(InFilePath, /*bAllowParentPaths*/!bIsFile);
	}

	return true;
}

bool FDefaultFileActions::GetItemAttribute(const FName InFilePath, const FString& InFilename, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	if (InAttributeKey == ContentBrowserItemAttributes::ItemIsDeveloperContent)
	{
		const bool bIsDevelopersFolder = AssetViewUtils::IsDevelopersFolder(FNameBuilder(InFilePath));
		OutAttributeValue.SetValue(bIsDevelopersFolder);
		return true;
	}

	if (InAttributeKey == ContentBrowserItemAttributes::ItemIsLocalizedContent)
	{
		const bool bIsLocalizedFolder = FPackageName::IsLocalizedPackage(FNameBuilder(InFilePath));
		OutAttributeValue.SetValue(bIsLocalizedFolder);
		return true;
	}

	if (InAttributeKey == ContentBrowserItemAttributes::ItemIsEngineContent)
	{
		const bool bIsEngineFolder = AssetViewUtils::IsEngineFolder(FNameBuilder(InFilePath), /*bIncludePlugins*/true);
		OutAttributeValue.SetValue(bIsEngineFolder);
		return true;
	}

	if (InAttributeKey == ContentBrowserItemAttributes::ItemIsProjectContent)
	{
		const bool bIsProjectFolder = AssetViewUtils::IsProjectFolder(FNameBuilder(InFilePath), /*bIncludePlugins*/true);
		OutAttributeValue.SetValue(bIsProjectFolder);
		return true;
	}

	if (InAttributeKey == ContentBrowserItemAttributes::ItemIsPluginContent)
	{
		const bool bIsPluginFolder = AssetViewUtils::IsPluginFolder(FNameBuilder(InFilePath));
		OutAttributeValue.SetValue(bIsPluginFolder);
		return true;
	}

	return false;
}

}

#undef LOCTEXT_NAMESPACE
