// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserFileDataPayload.h"
#include "Misc/PathViews.h"

#define LOCTEXT_NAMESPACE "ContentBrowserFileDataSource"

namespace ContentBrowserFileData
{

void FFileConfigData::SetDirectoryActions(const FDirectoryActions& InDirectoryActions)
{
	DirectoryActions = MakeShared<FDirectoryActions>(InDirectoryActions);
}

TSharedPtr<const FDirectoryActions> FFileConfigData::GetDirectoryActions()
{
	return DirectoryActions;
}

void FFileConfigData::RegisterFileActions(const FFileActions& InFileActions)
{
	checkf(!FileActionsMap.Contains(InFileActions.TypeExtension), TEXT("Extension '%s' was already registered!"), *InFileActions.TypeExtension);
	FileActionsMap.Add(InFileActions.TypeExtension, MakeShared<FFileActions>(InFileActions));
}

TSharedPtr<const FFileActions> FFileConfigData::FindFileActionsForExtension(FStringView InTypeExtension) const
{
	const TSharedPtr<const FFileActions>* FileActionsPtr = FileActionsMap.FindByHash(GetTypeHash(InTypeExtension), InTypeExtension);
	return FileActionsPtr ? *FileActionsPtr : nullptr;
}

TSharedPtr<const FFileActions> FFileConfigData::FindFileActionsForFilename(FStringView InFilename) const
{
	const FStringView FileExtension = FPathViews::GetExtension(InFilename);
	return FindFileActionsForExtension(FileExtension);
}

void FFileConfigData::EnumerateFileActions(TFunctionRef<bool(TSharedRef<const FFileActions>)> InCallback) const
{
	for (const auto& FileActionsPair : FileActionsMap)
	{
		if (!InCallback(FileActionsPair.Value.ToSharedRef()))
		{
			break;
		}
	}
}

FText FFileConfigData::GetDiscoveryDescription() const
{
	if (FileActionsMap.Num() > 0)
	{
		const auto& FileActionsPair = *FileActionsMap.begin();
		if (FileActionsMap.Num() == 1)
		{
			return FText::Format(LOCTEXT("DiscoveringFileType", "Discovering {0} files..."), FileActionsPair.Value->TypeDisplayName);
		}
		else
		{
			return FText::Format(LOCTEXT("DiscoveringFileTypes", "Discovering {0} (and {1} other) files..."), FileActionsPair.Value->TypeDisplayName, FileActionsMap.Num() - 1);
		}
	}

	return FText();
}

}

#undef LOCTEXT_NAMESPACE
