// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserFileDataSource.h"
#include "ContentBrowserFileDataCore.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Misc/PathViews.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDataUtils.h"
#include "AssetTypeActions_Base.h"
#include "ToolMenuSection.h"
#include "SourceControlHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserFileDataSource)

#define LOCTEXT_NAMESPACE "ContentBrowserFileDataSource"

DEFINE_LOG_CATEGORY_STATIC(LogContentBrowserFileDataSource, Warning, Warning);

namespace ContentBrowserFileDataSource
{
	FStringView GetRootOfPath(FStringView InPath)
	{
		const TCHAR* DataPtr = InPath.GetData();
		for (int32 i=1; i < InPath.Len(); ++i)
		{
			if (DataPtr[i] == TEXT('/'))
			{
				return InPath.Left(i);
			}
		}

		return InPath;
	}

	FString GetFilterNamePrefix()
	{
		return TEXT("ContentBrowserDataFilterFile");
	}
}

/** Asset type actions for files filter */
class FAssetTypeActions_FileDataSource : public FAssetTypeActions_Base
{
public:
	// Begin IAssetTypeActions interface
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Misc; }
	virtual bool IsImportedAsset() const override { return false; }

	virtual FText GetName() const override;
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override;
	virtual UClass* GetSupportedClass() const override;
	virtual FColor GetTypeColor() const override;
	virtual FName GetFilterName() const override;
	virtual FTopLevelAssetPath GetClassPathName() const override;
	// End IAssetTypeActions interface

	FName FilterName;
	FText Name;
	FText Description;
	FColor TypeColor;
	FTopLevelAssetPath ClassPathName;
};

FText FAssetTypeActions_FileDataSource::GetAssetDescription(const struct FAssetData& AssetData) const
{
	return Description;
}

UClass* FAssetTypeActions_FileDataSource::GetSupportedClass() const
{
	return nullptr;
}

FText FAssetTypeActions_FileDataSource::GetName() const
{
	return Name;
}

FColor FAssetTypeActions_FileDataSource::GetTypeColor() const
{
	return TypeColor;
}

FName FAssetTypeActions_FileDataSource::GetFilterName() const
{
	return FilterName;
}

FTopLevelAssetPath FAssetTypeActions_FileDataSource::GetClassPathName() const
{
	return ClassPathName;
}

class FContentBrowserFileDataDiscovery : public FRunnable
{
public:
	struct FDiscoveredItem
	{
		enum class EType : uint8
		{
			Directory,
			File,
		};

		EType Type;
		FString MountPath;
		FString DiskPath;
	};

	FContentBrowserFileDataDiscovery(const ContentBrowserFileData::FFileConfigData* InConfig);
	virtual ~FContentBrowserFileDataDiscovery() = default;

	//~ FRunnable interface
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

	/** Signals to end the thread and waits for it to close before returning */
	void EnsureCompletion();

	/** Gets search results from the file discovery */
	bool GetAndTrimSearchResults(TArray<FDiscoveredItem>& OutDiscoveredItems);

	/** Adds a path to the search queue */
	void AddPathToSearch(const FString& InMountPath, const FString& InDiskPath);

	/** Removes a path from the search queue, including any sub-paths and pending discovered items */
	void RemovePathFromSearch(const FString& InMountPath);

	/** If files are currently being asynchronously scanned in the specified path, this will cause them to be scanned before other files */
	void PrioritizeSearchPath(const FString& PathToPrioritize);

	/** True if in the process of discovering files in PathsToSearch */
	bool IsDiscoveringFiles() const;

private:
	struct FScanPath
	{
		FString MountPath;
		FString DiskPath;
	};

	/** Set of file types that we should consider */
	const ContentBrowserFileData::FFileConfigData* Config;

	/** A critical section to protect data transfer to other threads */
	mutable FCriticalSection WorkerThreadCriticalSection;

	/** True if in the process of discovering files in PathsToSearch */
	bool bIsDiscoveringFiles = false;

	/** The paths that are pending scan */
	TArray<FScanPath> PendingScanPaths;

	/** The items found during the search. It is not thread-safe to directly access this array */
	TArray<FDiscoveredItem> DiscoveredItems;

	/** > 0 if we've been asked to abort work in progress at the next opportunity */
	FThreadSafeCounter StopTaskCounter = 0;

	/** Thread to run the discovery FRunnable on */
	FRunnableThread* Thread = nullptr;
};

FContentBrowserFileDataDiscovery::FContentBrowserFileDataDiscovery(const ContentBrowserFileData::FFileConfigData* InConfig)
	: Config(InConfig)
{
	checkf(Config, TEXT("Config data cannot be null!"));
	Thread = FRunnableThread::Create(this, TEXT("FContentBrowserFileDataDiscovery"), 0, TPri_BelowNormal);
	checkf(Thread, TEXT("Failed to create content file data discovery thread"));
}

bool FContentBrowserFileDataDiscovery::Init()
{
	return true;
}

uint32 FContentBrowserFileDataDiscovery::Run()
{
	while (StopTaskCounter.GetValue() == 0)
	{
		FScanPath LocalScanPath;
		{
			FScopeLock CritSectionLock(&WorkerThreadCriticalSection);

			if (PendingScanPaths.Num() > 0)
			{
				bIsDiscoveringFiles = true;

				// Pop off the first path to search
				LocalScanPath = PendingScanPaths[0];
				PendingScanPaths.RemoveAt(0, EAllowShrinking::No);
			}
		}

		if (LocalScanPath.DiskPath.Len() > 0)
		{
			TArray<FScanPath> LocalPendingScanPaths;
			TArray<FDiscoveredItem> LocalDiscoveredItems;

			// Iterate the current search directory
			IFileManager::Get().IterateDirectory(*LocalScanPath.DiskPath, [this, &LocalScanPath, &LocalPendingScanPaths, &LocalDiscoveredItems](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
			{
				bool bIsValidItem = true;

				FString FilenameOrDirectoryStr = FPaths::ConvertRelativePathToFull(FilenameOrDirectory);
				if (!bIsDirectory)
				{
					bIsValidItem = Config->FindFileActionsForFilename(FilenameOrDirectoryStr).IsValid();
				}

				if (bIsValidItem)
				{
					FDiscoveredItem& DiscoveredItem = LocalDiscoveredItems.AddDefaulted_GetRef();
					DiscoveredItem.Type = bIsDirectory ? FDiscoveredItem::EType::Directory : FDiscoveredItem::EType::File;
					DiscoveredItem.DiskPath = MoveTemp(FilenameOrDirectoryStr);
					DiscoveredItem.MountPath = DiscoveredItem.DiskPath.Replace(*LocalScanPath.DiskPath, *LocalScanPath.MountPath);

					if (bIsDirectory)
					{
						LocalPendingScanPaths.Add(FScanPath{ DiscoveredItem.MountPath, DiscoveredItem.DiskPath });
					}
				}

				return StopTaskCounter.GetValue() == 0;
			});

			{
				FScopeLock CritSectionLock(&WorkerThreadCriticalSection);

				// Append any newly discovered sub-directories
				// We insert these at the start of the queue so that sub-directories of what we just scanned are processed first
				// which can help with disk locality and path prioritization
				PendingScanPaths.Insert(MoveTemp(LocalPendingScanPaths), 0);
				LocalPendingScanPaths.Reset();

				// Append any newly discovered items
				DiscoveredItems.Append(MoveTemp(LocalDiscoveredItems));
				LocalDiscoveredItems.Reset();
			}
		}
		else
		{
			{
				FScopeLock CritSectionLock(&WorkerThreadCriticalSection);
				bIsDiscoveringFiles = false;
			}

			// No work to do. Sleep for a little and try again later.
			FPlatformProcess::Sleep(0.1f);
		}
	}

	return 0;
}

void FContentBrowserFileDataDiscovery::Stop()
{
	StopTaskCounter.Increment();
}

void FContentBrowserFileDataDiscovery::Exit()
{
}

void FContentBrowserFileDataDiscovery::EnsureCompletion()
{
	{
		FScopeLock CritSectionLock(&WorkerThreadCriticalSection);
		PendingScanPaths.Reset();
	}

	Stop();

	if (Thread)
	{
		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;
	}
}

bool FContentBrowserFileDataDiscovery::GetAndTrimSearchResults(TArray<FDiscoveredItem>& OutDiscoveredItems)
{
	FScopeLock CritSectionLock(&WorkerThreadCriticalSection);

	OutDiscoveredItems.Append(MoveTemp(DiscoveredItems));
	DiscoveredItems.Reset();

	return bIsDiscoveringFiles;
}

void FContentBrowserFileDataDiscovery::AddPathToSearch(const FString& InMountPath, const FString& InDiskPath)
{
	FScopeLock CritSectionLock(&WorkerThreadCriticalSection);
	PendingScanPaths.Add(FScanPath{ InMountPath, InDiskPath });
}

void FContentBrowserFileDataDiscovery::RemovePathFromSearch(const FString& InMountPath)
{
	const FString MountPathRoot = InMountPath / FString();

	// TODO: There is a race condition here, where Run may have already discovered some new items 
	// from the path we're removing, and will add them to the list once we release this lock
	FScopeLock CritSectionLock(&WorkerThreadCriticalSection);

	// Scan paths may be the root itself or any of its children
	PendingScanPaths.RemoveAll([&InMountPath, &MountPathRoot](const FScanPath& InScanPath)
	{
		return InScanPath.MountPath == InMountPath 
			|| InScanPath.MountPath.StartsWith(MountPathRoot);
	});

	// Discovered items may only be children of the path
	DiscoveredItems.RemoveAll([&MountPathRoot](const FDiscoveredItem& InDiscoveredItem)
	{
		return InDiscoveredItem.MountPath.StartsWith(MountPathRoot);
	});
}

void FContentBrowserFileDataDiscovery::PrioritizeSearchPath(const FString& PathToPrioritize)
{
	FScopeLock CritSectionLock(&WorkerThreadCriticalSection);

	// Swap all priority paths to the top of the list
	if (PathToPrioritize.Len() > 0)
	{
		int32 LowestNonPriorityPathIdx = 0;
		for (int32 DirIdx = 0; DirIdx < PendingScanPaths.Num(); ++DirIdx)
		{
			if (PendingScanPaths[DirIdx].MountPath.StartsWith(PathToPrioritize))
			{
				PendingScanPaths.Swap(DirIdx, LowestNonPriorityPathIdx);
				LowestNonPriorityPathIdx++;
			}
		}
	}
}

bool FContentBrowserFileDataDiscovery::IsDiscoveringFiles() const
{
	FScopeLock CritSectionLock(&WorkerThreadCriticalSection);

	return bIsDiscoveringFiles;
}


void UContentBrowserFileDataSource::Initialize(const ContentBrowserFileData::FFileConfigData& InConfig, const bool InAutoRegister)
{
	Super::Initialize(InAutoRegister);

	Config = InConfig;

	if (FPlatformProcess::SupportsMultithreading())
	{
		BackgroundDiscovery = MakeShared<FContentBrowserFileDataDiscovery>(&Config);
	}
	else
	{
		UE_LOGF(LogContentBrowserFileDataSource, Error, "UContentBrowserFileDataSource '%ls': Unable to start data source, support for threads is required.", *GetFullName());
	}

	// Bind the asset specific menu extensions
	{
		FToolMenuOwnerScoped MenuOwnerScoped(GetFName());
		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AddNewContextMenu"))
		{
			Menu->AddDynamicSection(*FString::Printf(TEXT("DynamicSection_DataSource_%s"), *GetName()), FNewToolMenuDelegate::CreateLambda([WeakThis = TWeakObjectPtr<UContentBrowserFileDataSource>(this)](UToolMenu* InMenu)
			{
				if (UContentBrowserFileDataSource* This = WeakThis.Get())
				{
					This->PopulateAddNewContextMenu(InMenu);
				}
			}));
		}
	}

	// Register asset type actions
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	Config.EnumerateFileActions([this, &AssetTools](TSharedRef<const ContentBrowserFileData::FFileActions> InFileActions)
	{
		TSharedRef<FAssetTypeActions_FileDataSource> AssetTypeAction = MakeShared<FAssetTypeActions_FileDataSource>();
		AssetTypeAction->Name = InFileActions->TypeShortDescription;
		AssetTypeAction->Description = InFileActions->TypeFullDescription;
		AssetTypeAction->TypeColor = InFileActions->TypeColor.ToFColor(true);
		AssetTypeAction->FilterName = *(ContentBrowserFileDataSource::GetFilterNamePrefix() + InFileActions->TypeName.GetAssetName().ToString());
		AssetTypeAction->ClassPathName = InFileActions->TypeName;
		AssetTools.RegisterAssetTypeActions(AssetTypeAction);
		RegisteredAssetTypeActions.Add(AssetTypeAction);
		return true;
	});
}

void UContentBrowserFileDataSource::Shutdown()
{
	// Unregister asset type actions
	if (FAssetToolsModule* AssetToolsModule = FModuleManager::GetModulePtr<FAssetToolsModule>("AssetTools"))
	{
		for (const TSharedRef<IAssetTypeActions>& Action : RegisteredAssetTypeActions)
		{
			AssetToolsModule->Get().UnregisterAssetTypeActions(Action);
		}
	}

	UToolMenus::UnregisterOwner(GetFName());

	if (BackgroundDiscovery)
	{
		BackgroundDiscovery->EnsureCompletion();
		BackgroundDiscovery.Reset();
	}

	Super::Shutdown();
}

void UContentBrowserFileDataSource::AddFileMount(const FName InFileMountPath, const FString& InFileMountDiskPath)
{
	FString FileMountPath = InFileMountPath.ToString();
	checkf(FileMountPath.Len() > 0 && FileMountPath[FileMountPath.Len() - 1] != TEXT('/'), TEXT("File mount path '%s' should not be empty or have a trailing slash!"), *FileMountPath);

	FString FileMountDiskPath = FPaths::ConvertRelativePathToFull(InFileMountDiskPath);
	checkf(FileMountDiskPath.Len() > 0 && FileMountDiskPath[FileMountDiskPath.Len() - 1] != TEXT('/'), TEXT("File mount disk path '%s' should not be empty or have a trailing slash!"), *FileMountDiskPath);

	if (!ensureMsgf(!RegisteredFileMounts.Contains(InFileMountPath), TEXT("File mount '%s' already registered!"), *FileMountPath))
	{
		return;
	}

	FFileMount& RegisteredFileMount = RegisteredFileMounts.Add(InFileMountPath);
	RegisteredFileMount.DiskPath = FileMountDiskPath;

	// Add a directory item for this root
	AddDiscoveredItem(FDiscoveredItem::EType::Directory, FileMountPath, FileMountDiskPath, /*bIsRootPath*/true);

	// Add to virtual path tree
	{
		// Track subdirectories that share a common root
		const FStringView FileMountPathRoot = ContentBrowserFileDataSource::GetRootOfPath(FileMountPath);
		const FName FileMountPathRootName(FileMountPathRoot);
		TArray<FName>& Children = RegisteredFileMountRoots.FindOrAdd(FileMountPathRootName);
		Children.AddUnique(*FileMountPath);

		RootPathAdded(FileMountPathRoot);
	}

	// File mounts are always visible, even if empty
	OnAlwaysShowPath(FileMountPath);

	if (FPaths::DirectoryExists(FileMountDiskPath))
	{
		// Scan this new path
		if (BackgroundDiscovery)
		{
			BackgroundDiscovery->AddPathToSearch(FileMountPath, FileMountDiskPath);
		}

		// Watch this path for changes
		{
			static const FName NAME_DirectoryWatcher = "DirectoryWatcher";
			FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(NAME_DirectoryWatcher);
			if (IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get())
			{
				DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
					FileMountDiskPath, IDirectoryWatcher::FDirectoryChanged::CreateUObject(this, &UContentBrowserFileDataSource::OnDirectoryChanged, FileMountPath, FileMountDiskPath),
					RegisteredFileMount.DirectoryWatcherHandle,
					IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges
				);
			}
		}
	}
}

void UContentBrowserFileDataSource::RemoveFileMount(const FName InFileMountPath)
{
	// Remove from virtual path tree
	{
		FNameBuilder FileMountPathBuilder(InFileMountPath);
		const FStringView FileMountPathRoot = ContentBrowserFileDataSource::GetRootOfPath(FileMountPathBuilder);
		const FName FileMountPathRootName(FileMountPathRoot);

		bool bRemoveRoot = true;

		// Root path cannot be removed still has subdirectories
		if (TArray<FName>* Children = RegisteredFileMountRoots.Find(FileMountPathRootName))
		{
			Children->Remove(InFileMountPath);

			if (Children->Num() > 0)
			{
				bRemoveRoot = false;
			}
			else
			{
				RegisteredFileMountRoots.Remove(FileMountPathRootName);
			}
		}

		if (bRemoveRoot)
		{
			RootPathRemoved(FileMountPathRoot);
		}
	}

	FFileMount RegisteredFileMount;
	if (RegisteredFileMounts.RemoveAndCopyValue(InFileMountPath, RegisteredFileMount))
	{
		// Stop watching this path for changes
		{
			static const FName NAME_DirectoryWatcher = "DirectoryWatcher";
			FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(NAME_DirectoryWatcher);
			if (IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get())
			{
				DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(RegisteredFileMount.DiskPath, RegisteredFileMount.DirectoryWatcherHandle);
			}
		}

		// Remove from pending scan
		if (BackgroundDiscovery)
		{
			BackgroundDiscovery->RemovePathFromSearch(InFileMountPath.ToString());
		}
		
		// Remove the directory item for this root (and any remaining children)
		RemoveDiscoveredItem(InFileMountPath);
	}
}

bool UContentBrowserFileDataSource::HasFileMount(const FName InFileMountPath) const
{
	return RegisteredFileMounts.Contains(InFileMountPath);
}

void UContentBrowserFileDataSource::Tick(const float InDeltaTime)
{
	Super::Tick(InDeltaTime);

	if (BackgroundDiscovery)
	{
		TArray<FContentBrowserFileDataDiscovery::FDiscoveredItem> BackgroundDiscoveredItems;
		BackgroundDiscovery->GetAndTrimSearchResults(BackgroundDiscoveredItems);

		for (const FContentBrowserFileDataDiscovery::FDiscoveredItem& BackgroundDiscoveredItem : BackgroundDiscoveredItems)
		{
			AddDiscoveredItem(
				BackgroundDiscoveredItem.Type == FContentBrowserFileDataDiscovery::FDiscoveredItem::EType::Directory ? FDiscoveredItem::EType::Directory : FDiscoveredItem::EType::File, 
				BackgroundDiscoveredItem.MountPath, 
				BackgroundDiscoveredItem.DiskPath,
				/*bIsRootPath*/false
				);
		}
	}
}

void UContentBrowserFileDataSource::BuildRootPathVirtualTree()
{
	Super::BuildRootPathVirtualTree();

	for (const auto& RegisteredFileMount : RegisteredFileMounts)
	{
		RootPathAdded(ContentBrowserFileDataSource::GetRootOfPath(FNameBuilder(RegisteredFileMount.Key)));
	}
}

void UContentBrowserFileDataSource::CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserFileDataSource::CompileFilter);

	const FContentBrowserDataObjectFilter* ObjectFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataObjectFilter>();
	const FContentBrowserDataPackageFilter* PackageFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataPackageFilter>();
	const FContentBrowserDataClassFilter* ClassFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataClassFilter>();
	const FContentBrowserDataCollectionFilter* CollectionFilter = InFilter.ExtraFilters.FindFilter<FContentBrowserDataCollectionFilter>();

	const bool bIncludeFolders = EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders);
	const bool bIncludeFiles = EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles);

	const bool bIncludeMisc = EnumHasAnyFlags(InFilter.ItemCategoryFilter, EContentBrowserItemCategoryFilter::IncludeMisc);

	FContentBrowserDataFilterList& FilterList = OutCompiledFilter.CompiledFilters.FindOrAdd(this);
	FContentBrowserCompiledFileDataFilter& FileDataFilter = FilterList.FindOrAddFilter<FContentBrowserCompiledFileDataFilter>();

	// If we aren't including anything, then we can just bail now
	if (!bIncludeMisc || (!bIncludeFolders && !bIncludeFiles))
	{
		return;
	}

	TArray<FString> FileExtensionsToInclude;
	if (ClassFilter)
	{
		if (ClassFilter->ClassNamesToInclude.Num() > 0)
		{
			TArray<FString, TInlineAllocator<2>> ClassNamesToLookFor;
			TArray<FString, TInlineAllocator<2>> ClassFileExtensions;

			Config.EnumerateFileActions([&ClassNamesToLookFor, &ClassFileExtensions](TSharedRef<const ContentBrowserFileData::FFileActions> InFileActions)
			{
				TStringBuilder<64> ClassNameToLookFor;
				ClassNameToLookFor.Append(ContentBrowserFileDataSource::GetFilterNamePrefix());
				ClassNameToLookFor.Append(InFileActions->TypeName.ToString());
				ClassNamesToLookFor.Add(ClassNameToLookFor.ToString());

				ClassFileExtensions.Add(InFileActions->TypeExtension);
				return true;
			});

			// Determine if any are one of this instance's file data source
			for (const FString& ClassName : ClassFilter->ClassNamesToInclude)
			{
				int32 FoundIndex = INDEX_NONE;
				if (ClassNamesToLookFor.Find(ClassName, FoundIndex))
				{
					FileExtensionsToInclude.Add(ClassFileExtensions[FoundIndex]);
				}
			}
		}
	}

	// If we have any inclusive object, path, then skip files
	if ((ObjectFilter && (ObjectFilter->ObjectNamesToInclude.Num() > 0 || ObjectFilter->TagsAndValuesToInclude.Num() > 0)) ||
		(PackageFilter && (PackageFilter->PackageNamesToInclude.Num() > 0 || PackageFilter->PackagePathsToInclude.Num() > 0))
		)
	{
		return;
	}

	// When a class filter is active, only show files if a file filter is also active
	if (ClassFilter && (ClassFilter->ClassNamesToInclude.Num() > 0))
	{
		if (FileExtensionsToInclude.Num() == 0)
		{
			return;
		}
	}

	// TODO: Support adding loose files to collections via their internal mount paths - for now just bail
	if (CollectionFilter && CollectionFilter->Collections.Num() > 0)
	{
		return;
	}

	// Cache information necessary to recurse
	FileDataFilter.VirtualPathName = InPath;
	FileDataFilter.VirtualPath = InPath.ToString();
	FileDataFilter.bRecursivePaths = InFilter.bRecursivePaths;
	FileDataFilter.ItemAttributeFilter = InFilter.ItemAttributeFilter;
	FileDataFilter.FileExtensionsToInclude.Append(FileExtensionsToInclude);
	if (PackageFilter && PackageFilter->PathPermissionList && PackageFilter->PathPermissionList->HasFiltering())
	{
		FileDataFilter.PermissionList = PackageFilter->PathPermissionList;
	}
}

void UContentBrowserFileDataSource::EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	const FContentBrowserDataFilterList* FilterList = InFilter.CompiledFilters.Find(this);
	if (!FilterList)
	{
		return;
	}
	
	const FContentBrowserCompiledFileDataFilter* FileDataFilter = FilterList->FindFilter<FContentBrowserCompiledFileDataFilter>();
	if (!FileDataFilter)
	{
		return;
	}

	if (FileDataFilter->VirtualPath.Len() == 0)
	{
		return;
	}

	struct FLocalVisitor
	{
		FLocalVisitor(UContentBrowserFileDataSource* InDataSource, const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback, const FContentBrowserCompiledFileDataFilter& InFileDataFilter) :
			DataSource(InDataSource),
			Filter(InFilter),
			Callback(InCallback),
			FileDataFilter(InFileDataFilter)
		{
			bIncludeFolders = EnumHasAnyFlags(Filter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders);
			bIncludeFiles = EnumHasAnyFlags(Filter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles);
			bRecursivePaths = FileDataFilter.bRecursivePaths;
		}

	public:

		bool OnReachedInternalPath(const FName InPath, bool bCreateFolderItemForThisFolder, bool bNeedsFilterCheck)
		{
			const FDiscoveredItem* DiscoveredItem = DataSource->DiscoveredItems.Find(InPath);
			if (DiscoveredItem && DiscoveredItem->Type == UContentBrowserFileDataSource::FDiscoveredItem::EType::Directory)
			{
				int32 FolderDepth = 0;
				{
					FNameBuilder PathBuilder(InPath);
					if (bNeedsFilterCheck && !UContentBrowserFileDataSource::PassesFilters(PathBuilder, *DiscoveredItem, 0, FileDataFilter))
					{
						return true;
					}

					FolderDepth = ContentBrowserDataUtils::CalculateFolderDepthOfPath(PathBuilder);
				}

				if (bIncludeFolders && bCreateFolderItemForThisFolder)
				{
					if (!Callback(DataSource->CreateFolderItem(InPath, DiscoveredItem->DiskPath)))
					{
						return false;
					}

					if (!bRecursivePaths)
					{
						return true;
					}
				}

				return EnumerateSubPaths(InPath, *DiscoveredItem, FolderDepth);
			}

			return true;
		}

		static int32 CalculateFolderDepthOfPath(const FName InPath)
		{
			return ContentBrowserDataUtils::CalculateFolderDepthOfPath(FNameBuilder(InPath));
		}
	
		bool PassesFilters(const FName InPath) const
		{
			return DataSource->PassesFilters(InPath, 0, FileDataFilter);
		}

		/** Returns true if any internal root path under the given virtual path passes filters */
		bool FullyVirtualPathPassesFilters(const FName FullyVirtualPath) const
		{
			bool bAnyChildPassesFilters = false;

			DataSource->RootPathVirtualTree.EnumerateSubPaths(FullyVirtualPath, [this, &bAnyChildPassesFilters](FName VirtualSubPath, FName InternalSubPath)
			{
				if (bAnyChildPassesFilters == true)
				{
					// Stop enumeration
					return false;
				}
				else if (!InternalSubPath.IsNone() && PassesFilters(InternalSubPath))
				{
					// Stop enumeration
					bAnyChildPassesFilters = true;
					return false;
				}

				return true;
			}, true);

			return bAnyChildPassesFilters;
		}

		/**
		 * Enumerate and call function on each child item of a folder that passes filter
		 * 
		 * @param ItemMountPath	Mount path of folder item (eg, /Game/Scripts)
		 * @param InDiscoveredItem Folder from DataSource->DiscoveredItems TMap 
		 * @param InFolderDepthChecked Number of folders deep of ItemMountPath that have already been checked by PassesFilter (eg, 0 if none, 1 if /A, 2 if /A/B, etc..)
		 * @return True if enumerate was not stopped, False if enumerate should stop
		 */
		bool EnumerateSubPaths(const FName ItemMountPath, const FDiscoveredItem& InDiscoveredItem, const int32 InFolderDepthChecked)
		{
			for (const FName ChildItemName : InDiscoveredItem.ChildItems)
			{
				if (const FDiscoveredItem* DiscoveredChildItem = DataSource->DiscoveredItems.Find(ChildItemName))
				{
					if (DiscoveredChildItem->Type == FDiscoveredItem::EType::Directory)
					{
						if (!UContentBrowserFileDataSource::PassesFilters(FNameBuilder(ChildItemName), *DiscoveredChildItem, InFolderDepthChecked, FileDataFilter))
						{
							continue;
						}

						if (bIncludeFolders)
						{
							if (!Callback(DataSource->CreateFolderItem(ChildItemName, DiscoveredChildItem->DiskPath)))
							{
								return false;
							}
						}

						if (FileDataFilter.bRecursivePaths)
						{
							if (!EnumerateSubPaths(ChildItemName, *DiscoveredChildItem, InFolderDepthChecked + 1))
							{
								return false;
							}
						}
					}
					else if (DiscoveredChildItem->Type == FDiscoveredItem::EType::File)
					{
						if (bIncludeFiles && PassesFileTypeFilter(DiscoveredChildItem->DiskPath))
						{
							if (!Callback(DataSource->CreateFileItem(ChildItemName, DiscoveredChildItem->DiskPath)))
							{
								return false;
							}
						}
					}
				}
			}

			return true;
		};

		bool PassesFileTypeFilter(const FStringView DiskPath)
		{
			if (FileDataFilter.FileExtensionsToInclude.Num() == 0)
			{
				return true;
			}

			int32 LastDot = INDEX_NONE;
			if (DiskPath.FindLastChar(TEXT('.'), LastDot))
			{
				const FStringView FileExtension = DiskPath.Right(DiskPath.Len() - LastDot - 1);
				if (FileDataFilter.FileExtensionsToInclude.Contains(FileExtension))
				{
					return true;
				}
			}

			return false;
		}

		bool IncludeFolders() const
		{
			return bIncludeFolders;
		}

		bool IncludeFiles() const
		{
			return bIncludeFiles;
		}

	private:
		UContentBrowserFileDataSource* DataSource;
		const FContentBrowserDataCompiledFilter& Filter;
		TFunctionRef<bool(FContentBrowserItemData&&)> Callback;
		const FContentBrowserCompiledFileDataFilter& FileDataFilter;
		bool bIncludeFolders = false;
		bool bIncludeFiles = false;
		bool bRecursivePaths = false;
	};

	FLocalVisitor Visitor(this, InFilter, InCallback, *FileDataFilter);
	if (!Visitor.IncludeFolders() && !Visitor.IncludeFiles())
	{
		return;
	}

	RefreshVirtualPathTreeIfNeeded();

	FName ConvertedPath;
	const FName StartingVirtualPath = FileDataFilter->VirtualPathName;
	const EContentBrowserPathType ConvertedPathType = TryConvertVirtualPath(StartingVirtualPath, ConvertedPath);
	if (ConvertedPathType == EContentBrowserPathType::Internal)
	{
		Visitor.OnReachedInternalPath(ConvertedPath, /*bCreateFolderItemForThisFolder*/false, /*bNeedsFilterCheck*/true);
	}
	else if (ConvertedPathType == EContentBrowserPathType::Virtual)
	{
		if (FileDataFilter->bRecursivePaths)
		{
			// Virtual paths not supported by PassesFilters, enumerate internal paths in hierarchy and propagate results to virtual parents
			TSet<FName> VirtualPathsPassedFilter;
			VirtualPathsPassedFilter.Reserve(RootPathVirtualTree.NumPaths());
			RootPathVirtualTree.EnumerateSubPaths(StartingVirtualPath, [this, &Visitor, &VirtualPathsPassedFilter](FName VirtualSubPath, FName InternalSubPath)
			{
				if (!InternalSubPath.IsNone())
				{
					if (Visitor.PassesFilters(InternalSubPath))
					{
						// Propagate result to parents
						for (FName It = VirtualSubPath; !It.IsNone(); It = RootPathVirtualTree.GetParentPath(It))
						{
							bool bIsAlreadySet = false;
							VirtualPathsPassedFilter.Add(It, &bIsAlreadySet);
							if (bIsAlreadySet)
							{
								break;
							}
						}
					}
				}

				return true;
			}, /*recurse*/true);

			RootPathVirtualTree.EnumerateSubPaths(StartingVirtualPath, [this, &Visitor, &InCallback, &VirtualPathsPassedFilter](const FName VirtualSubPath, const FName InternalSubPath) 
			{
				if (VirtualPathsPassedFilter.Contains(VirtualSubPath))
				{
					if (!InternalSubPath.IsNone())
					{
						if (!Visitor.OnReachedInternalPath(InternalSubPath, /*bCreateFolderItemForThisFolder*/true, /*bNeedsFilterCheck*/false))
						{
							return false;
						}
					}
					else
					{
						if (Visitor.IncludeFolders())
						{
							if (!InCallback(CreateVirtualFolderItem(VirtualSubPath)))
							{
								return false;
							}
						}
					}
				}

				return true;
			}, /*recurse*/true);	
		}
		else
		{
			RootPathVirtualTree.EnumerateSubPaths(StartingVirtualPath, [this, &Visitor, &InCallback](const FName VirtualSubPath, const FName InternalSubPath) 
			{
				if (!InternalSubPath.IsNone())
				{
					if (!Visitor.OnReachedInternalPath(InternalSubPath, /*bCreateFolderItemForThisFolder*/true, /*bNeedsFilterCheck*/true))
					{
						return false;
					}
				}
				else
				{
					if (Visitor.IncludeFolders())
					{
						if (Visitor.FullyVirtualPathPassesFilters(VirtualSubPath))
						{
							if (!InCallback(CreateVirtualFolderItem(VirtualSubPath)))
							{
								return false;
							}
						}
					}
				}

				return true;
			}, /*recurse*/false);
		}
	}
}

void UContentBrowserFileDataSource::EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	FName InternalPath;
	if (!TryConvertVirtualPathToInternal(InPath, InternalPath))
	{
		return;
	}
	
	if (const FDiscoveredItem* DiscoveredItem = DiscoveredItems.Find(InternalPath))
	{
		if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders) && DiscoveredItem->Type == FDiscoveredItem::EType::Directory)
		{
			InCallback(CreateFolderItem(InternalPath, DiscoveredItem->DiskPath));
		}

		if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles) && DiscoveredItem->Type == FDiscoveredItem::EType::File)
		{
			InCallback(CreateFileItem(InternalPath, DiscoveredItem->DiskPath));
		}
	}
}

void UContentBrowserFileDataSource::EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	TStringBuilder<256> PathBuilder;
	PathBuilder.Append(InPath);
	FPathViews::NormalizeDirectoryName(PathBuilder);

	const FStringView Path(PathBuilder);
	// Find the RegisteredFileMounts the Path is under.
	for (const TPair<FName, FFileMount>& RegisteredFileMount : RegisteredFileMounts)
	{
		if (Path.StartsWith(RegisteredFileMount.Value.DiskPath) &&
			(Path.Len() == RegisteredFileMount.Value.DiskPath.Len() || Path[RegisteredFileMount.Value.DiskPath.Len()] == TEXT('/')))
		{
			TStringBuilder<256> InternalPathBuilder;
			RegisteredFileMount.Key.AppendString(InternalPathBuilder);
			InternalPathBuilder.Append(FStringView(Path).RightChop(RegisteredFileMount.Value.DiskPath.Len()));

			FName InternalPath(InternalPathBuilder, FNAME_Find);
			if (!InternalPath.IsNone())
			{
				if (const FDiscoveredItem* DiscoveredItem = DiscoveredItems.Find(InternalPath))
				{
					switch (DiscoveredItem->Type)
					{
					case FDiscoveredItem::EType::Directory:
						if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
						{
							InCallback(CreateFolderItem(InternalPath, DiscoveredItem->DiskPath));
						}
						break;
					case FDiscoveredItem::EType::File:
						if (EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles))
						{
							InCallback(CreateFileItem(InternalPath, DiscoveredItem->DiskPath));
						}
						break;
					}

					return;
				}
			}
		}
	}
}

bool UContentBrowserFileDataSource::CanCreateFolder(const FName InPath, FText* OutErrorMsg)
{
	FName InternalPath;
	if (!TryConvertVirtualPathToInternal(InPath, InternalPath))
	{
		return false;
	}

	FString InternalDiskPath;
	if (!IsKnownFileMount(InternalPath, &InternalDiskPath))
	{
		return false;
	}

	if (TSharedPtr<const ContentBrowserFileData::FDirectoryActions> DirectoryActions = Config.GetDirectoryActions())
	{
		if (DirectoryActions->CanCreate.IsBound() && !DirectoryActions->CanCreate.Execute(InternalPath, InternalDiskPath, OutErrorMsg))
		{
			return false;
		}
	}

	return ContentBrowserFileData::CanModifyDirectory(InternalDiskPath, OutErrorMsg);
}

bool UContentBrowserFileDataSource::CreateFolder(const FName InPath, const TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>& HideFolderIfEmptyFilter, FContentBrowserItemDataTemporaryContext& OutPendingItem)
{
	FName InternalPath;
	if (!TryConvertVirtualPathToInternal(InPath, InternalPath))
	{
		return false;
	}

	FString InternalDiskPath;
	if (!IsKnownFileMount(InternalPath, &InternalDiskPath))
	{
		return false;
	}

	const FString FolderItemName = FPaths::GetCleanFilename(InternalDiskPath);

	FContentBrowserItemData NewItemData(
		this,
		EContentBrowserItemFlags::Type_Folder | EContentBrowserItemFlags::Category_Misc
			| EContentBrowserItemFlags::Temporary_Creation,
		InPath,
		*FolderItemName,
		FText::AsCultureInvariant(FolderItemName),
		MakeShared<FContentBrowserFolderItemDataPayload>(InternalPath, InternalDiskPath, Config.GetDirectoryActions()),
		{ InternalPath });

	OutPendingItem = FContentBrowserItemDataTemporaryContext(
		MoveTemp(NewItemData),
		FContentBrowserItemDataTemporaryContext::FOnValidateItem::CreateLambda([DataSource = TWeakObjectPtr<UContentBrowserFileDataSource>(this), HideFolderIfEmptyFilter](const FContentBrowserItemData& InItem, const FString& InProposedName, FText* OutErrorMsg)
			{
				if (UContentBrowserFileDataSource* DataSourcePtr = DataSource.Get())
				{
					return DataSourcePtr->CanRenameItem(InItem, &InProposedName, HideFolderIfEmptyFilter.Get(), OutErrorMsg);
				}
				return false;
			}),
		FContentBrowserItemDataTemporaryContext::FOnFinalizeItem::CreateUObject(
			this, &UContentBrowserFileDataSource::OnFinalizeCreateFolder));

	return true;
}

bool UContentBrowserFileDataSource::IsDiscoveringItems(FText* OutStatus)
{
	if (BackgroundDiscovery && BackgroundDiscovery->IsDiscoveringFiles())
	{
		ContentBrowserFileData::SetOptionalErrorMessage(OutStatus, Config.GetDiscoveryDescription());
		return true;
	}

	return false;
}

bool UContentBrowserFileDataSource::PrioritizeSearchPath(const FName InPath)
{
	FName InternalPath;
	if (!TryConvertVirtualPathToInternal(InPath, InternalPath))
	{
		return false;
	}

	if (BackgroundDiscovery)
	{
		BackgroundDiscovery->PrioritizeSearchPath(InternalPath.ToString());
	}
	return true;
}

bool UContentBrowserFileDataSource::IsFolderVisible(const FName InPath, const EContentBrowserIsFolderVisibleFlags InFlags, const FContentBrowserFolderContentsFilter& ContentsFilter)
{
	if (!EnumHasAllFlags(ContentsFilter.ItemCategoryFilter, EContentBrowserItemCategoryFilter::IncludeMisc))
	{
		return false;
	}

	FName ConvertedPath;
	const EContentBrowserPathType ConvertedPathType = TryConvertVirtualPath(InPath, ConvertedPath);
	if (ConvertedPathType == EContentBrowserPathType::Internal)
	{
		if (!IsKnownFileMount(ConvertedPath))
		{
			return false;
		}
	}
	else if (ConvertedPathType == EContentBrowserPathType::Virtual)
	{
		return true;
	}
	else
	{
		return false;
	}

	FNameBuilder InternalPathStr(ConvertedPath);
	const FStringView InternalPathStrView = InternalPathStr;
	const uint32 InternalPathHash = GetTypeHash(InternalPathStrView);

	if (AlwaysVisibleFolders.ContainsByHash(InternalPathHash, InternalPathStrView))
	{
		return true;
	}
	// Misc flag was checked above - if we are filtering out folders that don't contain "misc" elements, we are filtering out all empty folders from this provider,
	// unless a custom empty folder filter is provided, in which case we use that
	if (EmptyFolders.ContainsByHash(InternalPathHash, InternalPathStrView))
	{
		if (ContentsFilter.HideFolderIfEmptyFilter)
		{
			return !HideFolderIfEmpty(*ContentsFilter.HideFolderIfEmptyFilter, ConvertedPath, InternalPathStrView);
		}
		return false;
	}
	return true;
}

bool UContentBrowserFileDataSource::DoesItemPassFilter(const FContentBrowserItemData& InItem, const FContentBrowserDataCompiledFilter& InFilter)
{
	const FContentBrowserDataFilterList* FilterList = InFilter.CompiledFilters.Find(this);
	if (!FilterList)
	{
		return false;
	}

	const FContentBrowserCompiledFileDataFilter* FileDataFilter = FilterList->FindFilter<FContentBrowserCompiledFileDataFilter>();
	if (!FileDataFilter)
	{
		return false;
	}

	FName InternalPath;
	switch (InItem.GetItemType())
	{
	case EContentBrowserItemFlags::Type_Folder:
		if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
		{
			if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InItem))
			{
				InternalPath = FolderPayload->GetInternalPath();
			}
		}
		break;

	case EContentBrowserItemFlags::Type_File:
		if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFiles))
		{
			if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = GetFileItemPayload(InItem))
			{
				InternalPath = FilePayload->GetInternalPath();
			}
		}
		break;

	default:
		break;
	}

	if (!InternalPath.IsNone())
	{
		const FName VirtualPath = InItem.GetVirtualPath();
		FNameBuilder VirtualPathBuilder(VirtualPath);
		const FStringView VirtualPathView(VirtualPathBuilder);

		if (VirtualPathView.StartsWith(FileDataFilter->VirtualPath))
		{
			const bool bIsExactMatch = VirtualPathView.Len() <= FileDataFilter->VirtualPath.Len();

			// Do not include the filter's folder itself
			if (bIsExactMatch)
			{
				return false;
			}

			if (VirtualPathView[FileDataFilter->VirtualPath.Len()] == TEXT('/'))
			{
				bool bIsUnderSearchPath = false;
				if (FileDataFilter->bRecursivePaths)
				{
					bIsUnderSearchPath = true;
				}
				else
				{
					const FStringView RemainingView = VirtualPathView.RightChop(FileDataFilter->VirtualPath.Len() + 1);
					int32 FoundIndex = INDEX_NONE;
					if (!RemainingView.FindChar(TEXT('/'), FoundIndex))
					{
						bIsUnderSearchPath = true;
					}
				}

				if (bIsUnderSearchPath)
				{
					if (InItem.GetItemType() == EContentBrowserItemFlags::Type_Folder)
					{
						return PassesFilters(InternalPath, 0, *FileDataFilter);
					}
					else if (InItem.GetItemType() == EContentBrowserItemFlags::Type_File)
					{
						return PassesFilters(InternalPath, 0, *FileDataFilter);
					}
				}
			}
		}
	}

	return false;
}

bool UContentBrowserFileDataSource::PassesFilters(const FStringView InPath, const FDiscoveredItem& InDiscoveredItem, const int32 InFolderDepthChecked, const FContentBrowserCompiledFileDataFilter& InFileDataFilter)
{
	if (InFileDataFilter.PermissionList.IsValid())
	{
		if (!InFileDataFilter.PermissionList->PassesStartsWithFilter(InPath, /*bAllowParentPaths*/ InDiscoveredItem.Type == UContentBrowserFileDataSource::FDiscoveredItem::EType::File))
		{
			return false;
		}
	}

	if (!ContentBrowserDataUtils::PathPassesAttributeFilter(InPath, InFolderDepthChecked, InFileDataFilter.ItemAttributeFilter))
	{
		return false;
	}

	return true;
}

bool UContentBrowserFileDataSource::PassesFilters(const FName InPath, const int32 InFolderDepthChecked, const FContentBrowserCompiledFileDataFilter& InFileDataFilter) const
{
	if (const FDiscoveredItem* DiscoveredItem = DiscoveredItems.Find(InPath))
	{
		return PassesFilters(FNameBuilder(InPath), *DiscoveredItem, InFolderDepthChecked, InFileDataFilter);
	}

	return false;
}

bool UContentBrowserFileDataSource::GetItemAttribute(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	return ContentBrowserFileData::GetItemAttribute(this, InItem, InIncludeMetaData, InAttributeKey, OutAttributeValue);
}

bool UContentBrowserFileDataSource::GetItemAttributes(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	return ContentBrowserFileData::GetItemAttributes(this, InItem, InIncludeMetaData, OutAttributeValues);
}

bool UContentBrowserFileDataSource::GetItemPhysicalPath(const FContentBrowserItemData& InItem, FString& OutDiskPath)
{
	return ContentBrowserFileData::GetItemPhysicalPath(this, InItem, OutDiskPath);
}

bool UContentBrowserFileDataSource::CanEditItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserFileData::CanEditItem(this, InItem, OutErrorMsg);
}

bool UContentBrowserFileDataSource::EditItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserFileData::EditItems(this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserFileDataSource::BulkEditItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserFileData::EditItems(this, InItems);
}

bool UContentBrowserFileDataSource::CanPreviewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserFileData::CanPreviewItem(this, InItem, OutErrorMsg);
}

bool UContentBrowserFileDataSource::PreviewItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserFileData::PreviewItems(this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserFileDataSource::BulkPreviewItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserFileData::PreviewItems(this, InItems);
}

bool UContentBrowserFileDataSource::CanDuplicateItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserFileData::CanDuplicateItem(this, InItem, OutErrorMsg);
}

bool UContentBrowserFileDataSource::DuplicateItem(const FContentBrowserItemData& InItem, FContentBrowserItemDataTemporaryContext& OutPendingItem)
{
	TSharedPtr<const FContentBrowserFileItemDataPayload_Duplication> NewItemPayload;
	if (ContentBrowserFileData::DuplicateItem(this, InItem, NewItemPayload))
	{
		checkf(NewItemPayload, TEXT("DuplicateItem returned true, but NewItemPayload was null!"));

		FName VirtualizedPath;
		FName InternalPath = NewItemPayload->GetInternalPath();
		TryConvertInternalPathToVirtual(InternalPath, VirtualizedPath);

		const FString FileItemName = FPaths::GetBaseFilename(NewItemPayload->GetFilename());

		FContentBrowserItemData NewItemData(
			this,
			EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Category_Misc
				| EContentBrowserItemFlags::Temporary_Duplication,
			VirtualizedPath,
			*FileItemName,
			FText::AsCultureInvariant(FileItemName),
			NewItemPayload,
			{ InternalPath });

		OutPendingItem = FContentBrowserItemDataTemporaryContext(
			MoveTemp(NewItemData),
			FContentBrowserItemDataTemporaryContext::FOnValidateItem::CreateUObject(
				this, &UContentBrowserFileDataSource::OnValidateItemName),
			FContentBrowserItemDataTemporaryContext::FOnFinalizeItem::CreateUObject(
				this, &UContentBrowserFileDataSource::OnFinalizeDuplicateFile));

		return true;
	}

	return false;
}

bool UContentBrowserFileDataSource::BulkDuplicateItems(TArrayView<const FContentBrowserItemData> InItems, TArray<FContentBrowserItemData>& OutNewItems)
{
	TArray<TSharedRef<const FContentBrowserFileItemDataPayload>> NewItemPayloads;
	if (ContentBrowserFileData::DuplicateItems(this, InItems, NewItemPayloads))
	{
		for (const TSharedRef<const FContentBrowserFileItemDataPayload>& NewItemPayload : NewItemPayloads)
		{
			FName VirtualizedPath;
			FName InternalPath = NewItemPayload->GetInternalPath();
			TryConvertInternalPathToVirtual(InternalPath, VirtualizedPath);

			const FString FileItemName = FPaths::GetBaseFilename(NewItemPayload->GetFilename());

			OutNewItems.Emplace(FContentBrowserItemData(
				this,
				EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Category_Misc,
				VirtualizedPath,
				*FileItemName,
				FText(),
				NewItemPayload,
				{ InternalPath }));
		}

		return true;
	}

	return false;
}

bool UContentBrowserFileDataSource::CanDeleteItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return ContentBrowserFileData::CanDeleteItem(this, InItem, OutErrorMsg);
}

bool UContentBrowserFileDataSource::DeleteItem(const FContentBrowserItemData& InItem)
{
	return ContentBrowserFileData::DeleteItems(this, MakeArrayView(&InItem, 1));
}

bool UContentBrowserFileDataSource::BulkDeleteItems(TArrayView<const FContentBrowserItemData> InItems)
{
	return ContentBrowserFileData::DeleteItems(this, InItems);
}

bool UContentBrowserFileDataSource::CanRenameItem(const FContentBrowserItemData& InItem, const FString* InNewName, const IContentBrowserHideFolderIfEmptyFilter* HideFolderIfEmptyFilter, FText* OutErrorMsg)
{
	// Cannot copy an item outside the paths known to this data source
	FName InternalPath;
	if (!TryConvertVirtualPathToInternal(InItem.GetVirtualPath(), InternalPath))
	{
		return false;
	}

	// We cannot rename the file mounts themselves
	if (IsRootFileMount(InternalPath))
	{
		ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsFileRoot", "Folder '{0}' is a root file path and cannot be renamed"), FText::FromName(InternalPath)));
		return false;
	}

	// We cannot rename things outside of our file mounts
	if (!IsKnownFileMount(InternalPath))
	{
		ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsNotFilePath", "Folder '{0}' is not a known file path"), FText::FromName(InternalPath)));
		return false;
	}

	bool bCheckUniqueName = true;

	if (InNewName != nullptr && HideFolderIfEmptyFilter != nullptr)
	{
		if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InItem))
		{
			FString NewInternalPath = FPaths::GetPath(FolderPayload->GetInternalPath().ToString()) / *InNewName;
			const uint32 NewInternalPathHash = GetTypeHash(NewInternalPath);

			if (!AlwaysVisibleFolders.ContainsByHash(NewInternalPathHash, NewInternalPath) &&
				EmptyFolders.ContainsByHash(NewInternalPathHash, NewInternalPath) &&
				HideFolderIfEmpty(*HideFolderIfEmptyFilter, FName(NewInternalPath), NewInternalPath))
			{
				// We are renaming to an existing hidden folder name, disable the unique name check since it will fail.
				bCheckUniqueName = false;
			}
		}
	}

	return ContentBrowserFileData::CanRenameItem(this, InItem, bCheckUniqueName, InNewName, OutErrorMsg);
}

bool UContentBrowserFileDataSource::RenameItem(const FContentBrowserItemData& InItem, const FString& InNewName, FContentBrowserItemData& OutNewItem)
{
	FName NewInternalPath;
	FString NewFilename;
	if (ContentBrowserFileData::RenameItem(Config, this, InItem, InNewName, NewInternalPath, NewFilename))
	{
		switch (InItem.GetItemType())
		{
		case EContentBrowserItemFlags::Type_Folder:
			OutNewItem = CreateFolderItem(NewInternalPath, NewFilename);
			break;

		case EContentBrowserItemFlags::Type_File:
			OutNewItem = CreateFileItem(NewInternalPath, NewFilename);
			break;

		default:
			break;
		}

		return true;
	}

	return false;
}

bool UContentBrowserFileDataSource::CanCopyItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg)
{
	// Cannot copy an item outside the paths known to this data source
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsUnknown", "Folder '{0}' is outside the mount roots of this data source"), FText::FromName(InDestPath)));
		return false;
	}

	// The destination path must be a content folder
	FString InternalDestDiskPath;
	if (!IsKnownFileMount(InternalDestPath, &InternalDestDiskPath))
	{
		ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsNotFilePath", "Folder '{0}' is not a known file path"), FText::FromName(InDestPath)));
		return false;
	}

	// The destination path must be writable
	if (!ContentBrowserFileData::CanModifyDirectory(InternalDestDiskPath, OutErrorMsg))
	{
		return false;
	}

	// Destination must not be self (folder)
	const FName VirtualPath = InItem.GetVirtualPath();
	if (InDestPath == VirtualPath)
	{
		ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("CopyError_ItemInsideItself", "Can't copy folder inside itself"));
		return false;
	}
	return true;
}

bool UContentBrowserFileDataSource::CopyItem(const FContentBrowserItemData& InItem, const FName InDestPath)
{
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		return false;
	}

	FString InternalDestDiskPath;
	if (!IsKnownFileMount(InternalDestPath, &InternalDestDiskPath))
	{
		return false;
	}

	return ContentBrowserFileData::CopyItems(Config, this, MakeArrayView(&InItem, 1), InternalDestDiskPath);
}

bool UContentBrowserFileDataSource::BulkCopyItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath)
{
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		return false;
	}

	FString InternalDestDiskPath;
	if (!IsKnownFileMount(InternalDestPath, &InternalDestDiskPath))
	{
		return false;
	}

	return ContentBrowserFileData::CopyItems(Config, this, InItems, InternalDestDiskPath);
}

bool UContentBrowserFileDataSource::CanMoveItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg)
{
	// Cannot move an item outside the paths known to this data source
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsUnknown", "Folder '{0}' is outside the mount roots of this data source"), FText::FromName(InDestPath)));
		return false;
	}

	// The destination path must be a content folder
	FString InternalDestDiskPath;
	if (!IsKnownFileMount(InternalDestPath, &InternalDestDiskPath))
	{
		ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, FText::Format(LOCTEXT("Error_FolderIsNotFilePath", "Folder '{0}' is not a known file path"), FText::FromName(InDestPath)));
		return false;
	}

	// The destination path must be writable
	if (!ContentBrowserFileData::CanModifyDirectory(InternalDestDiskPath, OutErrorMsg))
	{
		return false;
	}

	// Moving has to be able to delete the original item
	if (!ContentBrowserFileData::CanModifyItem(this, InItem, OutErrorMsg))
	{
		return false;
	}

	
	// Destination must not be self (folder)
	FName VirtualPath = InItem.GetVirtualPath();
	if (InDestPath == VirtualPath)
	{
		ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("MoveError_ItemInsideItself", "Can't move Item inside itself"));
		return false;
	}

	// Cannot be moved to the same folder
	FString VirtualPathAsString = VirtualPath.ToString();
	int32 LastSlashIndex = INDEX_NONE;
	VirtualPathAsString.FindLastChar('/', LastSlashIndex);
	if (LastSlashIndex != INDEX_NONE)
	{
		VirtualPathAsString = VirtualPathAsString.Left(VirtualPathAsString.Len() - (VirtualPathAsString.Len() - LastSlashIndex));
	}

	if (InDestPath == FName(VirtualPathAsString))
	{
		ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("MoveError_ItemInsideSamePath", "Can't move Item inside the same location"));
		return false;
	}

	return true;
}

bool UContentBrowserFileDataSource::MoveItem(const FContentBrowserItemData& InItem, const FName InDestPath)
{
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		return false;
	}

	FString InternalDestDiskPath;
	if (!IsKnownFileMount(InternalDestPath, &InternalDestDiskPath))
	{
		return false;
	}

	return ContentBrowserFileData::MoveItems(Config, this, MakeArrayView(&InItem, 1), InternalDestDiskPath);
}

bool UContentBrowserFileDataSource::BulkMoveItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath)
{
	FName InternalDestPath;
	if (!TryConvertVirtualPathToInternal(InDestPath, InternalDestPath))
	{
		return false;
	}

	FString InternalDestDiskPath;
	if (!IsKnownFileMount(InternalDestPath, &InternalDestDiskPath))
	{
		return false;
	}

	return ContentBrowserFileData::MoveItems(Config, this, InItems, InternalDestDiskPath);
}

bool UContentBrowserFileDataSource::UpdateThumbnail(const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail)
{
	return ContentBrowserFileData::UpdateItemThumbnail(this, InItem, InThumbnail);
}

FContentBrowserItemData UContentBrowserFileDataSource::CreateFolderItem(const FName InInternalPath, const FString& InFilename)
{
	FName VirtualizedPath;
	TryConvertInternalPathToVirtual(InInternalPath, VirtualizedPath);

	FString AlternativeName;
	if (InFilename.IsEmpty())
	{
		AlternativeName = FPackageName::GetShortName(VirtualizedPath);
	}

	return ContentBrowserFileData::CreateFolderItem(this, VirtualizedPath, InInternalPath, InFilename.IsEmpty() ? AlternativeName : InFilename, Config.GetDirectoryActions());
}

FContentBrowserItemData UContentBrowserFileDataSource::CreateFileItem(const FName InInternalPath, const FString& InFilename)
{
	FName VirtualizedPath;
	TryConvertInternalPathToVirtual(InInternalPath, VirtualizedPath);

	return ContentBrowserFileData::CreateFileItem(this, VirtualizedPath, InInternalPath, InFilename, Config.FindFileActionsForFilename(InFilename));
}

FContentBrowserItemData UContentBrowserFileDataSource::CreateItemFromDiscovered(const FName InInternalPath, const FDiscoveredItem& InDiscoveredItem)
{
	if (InDiscoveredItem.Type == FDiscoveredItem::EType::Directory)
	{
		return CreateFolderItem(InInternalPath, InDiscoveredItem.DiskPath);
	}
	return CreateFileItem(InInternalPath, InDiscoveredItem.DiskPath);
}

TSharedPtr<const FContentBrowserFolderItemDataPayload> UContentBrowserFileDataSource::GetFolderItemPayload(const FContentBrowserItemData& InItem) const
{
	return ContentBrowserFileData::GetFolderItemPayload(this, InItem);
}

TSharedPtr<const FContentBrowserFileItemDataPayload> UContentBrowserFileDataSource::GetFileItemPayload(const FContentBrowserItemData& InItem) const
{
	return ContentBrowserFileData::GetFileItemPayload(this, InItem);
}

bool UContentBrowserFileDataSource::HideFolderIfEmpty(const IContentBrowserHideFolderIfEmptyFilter& HideFolderIfEmptyFilter, FName Path, FStringView PathString) const
{
	check(EmptyFolders.ContainsByHash(GetTypeHash(PathString), PathString));

	if (!HideFolderIfEmptyFilter.HideFolderIfEmpty(Path, PathString))
	{
		return false;
	}

	if (const FDiscoveredItem* DiscoveredItem = DiscoveredItems.Find(Path))
	{
		checkf(DiscoveredItem->Type == FDiscoveredItem::EType::Directory, TEXT("Empty folders should not contain files."));

		for (const FName ChildItem : DiscoveredItem->ChildItems)
		{
			FNameBuilder ChildPathBuilder(ChildItem);
			if (!HideFolderIfEmpty(HideFolderIfEmptyFilter, ChildItem, ChildPathBuilder))
			{
				// If any subpaths shouldn't be hidden, then Path should be visible.
				return false;
			}
		}
	}

	return true;
}

bool UContentBrowserFileDataSource::IsKnownFileMount(const FName InMountPath, FString* OutDiskPath) const
{
	FNameBuilder MountPathStr(InMountPath);

	const FStringView MountPathStrView = MountPathStr;
	for (const auto& RegisteredFileMount : RegisteredFileMounts)
	{
		FNameBuilder FileMountMountRootStr(RegisteredFileMount.Key);

		const FStringView FileMountMountRootStrView = FileMountMountRootStr;
		if (MountPathStrView.StartsWith(FileMountMountRootStrView, ESearchCase::IgnoreCase) 
			&& (MountPathStrView.Len() == FileMountMountRootStrView.Len() || (MountPathStrView.Len() > FileMountMountRootStrView.Len() && MountPathStrView[FileMountMountRootStrView.Len()] == TEXT('/')))
			)
		{
			if (OutDiskPath)
			{
				*OutDiskPath = MountPathStrView;
				OutDiskPath->RightChopInline(FileMountMountRootStr.Len());

				FString Path = FPaths::ConvertRelativePathToFull(*FileMountMountRootStr, *RegisteredFileMount.Value.DiskPath);
				*OutDiskPath = Path.Append(*OutDiskPath); 
			}
			return true;
		}
	}

	return false;
}

bool UContentBrowserFileDataSource::IsRootFileMount(const FName InMountPath, FString* OutDiskPath) const
{
	if (const FFileMount* RegisteredFileMount = RegisteredFileMounts.Find(InMountPath))
	{
		if (OutDiskPath)
		{
			*OutDiskPath = RegisteredFileMount->DiskPath;
		}

		return true;
	}

	return false;
}

void UContentBrowserFileDataSource::AddDiscoveredItem(FDiscoveredItem::EType InItemType, const FString& InMountPath, const FString& InDiskPath, const bool bIsRootPath)
{
	AddDiscoveredItemImpl(InItemType, InMountPath, InDiskPath, NAME_None, bIsRootPath);
}

void UContentBrowserFileDataSource::AddDiscoveredItemImpl(FDiscoveredItem::EType InItemType, const FString& InMountPath, const FString& InDiskPath, const FName InChildMountPathName, const bool bIsRootPath)
{
	const FName MountPathName = *InMountPath;

	// If this item already exists then just add the required child item (if any)
	if (FDiscoveredItem* DiscoveredItemPtr = DiscoveredItems.Find(MountPathName))
	{
		if (!InChildMountPathName.IsNone())
		{
			checkf(DiscoveredItemPtr->Type == FDiscoveredItem::EType::Directory, TEXT("Only directory items may have children!"));
			DiscoveredItemPtr->ChildItems.Add(InChildMountPathName);
		}
		if (DiscoveredItemPtr->Type == FDiscoveredItem::EType::File)
		{
			// Notify that this file was modified
			QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateItemFromDiscovered(MountPathName, *DiscoveredItemPtr)));
		}
		return;
	}

	// Add this item since it doesn't exist yet
	{
		FDiscoveredItem& DiscoveredItem = DiscoveredItems.Add(MountPathName);
		DiscoveredItem.Type = InItemType;
		DiscoveredItem.DiskPath = InDiskPath;
		if (!InChildMountPathName.IsNone())
		{
			checkf(DiscoveredItem.Type == FDiscoveredItem::EType::Directory, TEXT("Only directory items may have children!"));
			DiscoveredItem.ChildItems.Add(InChildMountPathName);
		}

		// Directories are considered empty until files are found within them
		if (InItemType == FDiscoveredItem::EType::Directory)
		{
			EmptyFolders.Add(InMountPath);
		}
	}

	// Add any missing parents too
	// Note: If this is a root path then we don't try and guess what it's parent disk path would be, as it would exist outside of any place we have been asked to monitor
	if (InMountPath != TEXT("/"))
	{
		FString ParentMountPath = FPaths::GetPath(InMountPath);
		if (ParentMountPath.IsEmpty())
		{
			// GetPath returns an empty string when given a top-level path like "/Game" so we set it to "/" instead
			ParentMountPath = TEXT("/");
		}

		// Note that the parent directory now contains content
		if (InItemType == FDiscoveredItem::EType::File)
		{
			OnPathPopulated(ParentMountPath);
		}

		const FString ParentDiskPath = bIsRootPath ? FString() : FPaths::GetPath(InDiskPath);
		AddDiscoveredItemImpl(FDiscoveredItem::EType::Directory, ParentMountPath, ParentDiskPath, MountPathName, bIsRootPath);
	}

	// Notify that this item exists
	// Note: This happens after adding the parents so that the notifications are processed in the correct parent->child creation order
	FDiscoveredItem& DiscoveredItem = DiscoveredItems.FindChecked(MountPathName);
	QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemAddedUpdate(CreateItemFromDiscovered(MountPathName, DiscoveredItem)));
}

void UContentBrowserFileDataSource::RemoveDiscoveredItem(const FString& InMountPath)
{
	RemoveDiscoveredItemImpl(*InMountPath, /*bParentIsOrphan*/false);
}

void UContentBrowserFileDataSource::RemoveDiscoveredItem(const FName InMountPath)
{
	RemoveDiscoveredItemImpl(InMountPath, /*bParentIsOrphan*/false);
}

void UContentBrowserFileDataSource::RemoveDiscoveredItemImpl(const FName InMountPath, const bool bParentIsOrphan)
{
	// Remove this item from the map, which will orphan its entire tree from being able to find their parent item
	FDiscoveredItem DiscoveredItem;
	if (!DiscoveredItems.RemoveAndCopyValue(InMountPath, DiscoveredItem))
	{
		// If this item doesn't exist then we have nothing more to do
		return;
	}

	// Unlink this item from its parent, if the parent item isn't itself being orphaned
	if (!bParentIsOrphan)
	{
		FString ParentMountPath = FPaths::GetPath(InMountPath.ToString());
		if (ParentMountPath.IsEmpty())
		{
			// GetPath returns an empty string when given a top-level path like "/Game" so we set it to "/" instead
			ParentMountPath = TEXT("/");
		}

		if (FDiscoveredItem* ParentItemPtr = DiscoveredItems.Find(*ParentMountPath))
		{
			ParentItemPtr->ChildItems.Remove(InMountPath);
		}
	}

	// Remove any children of this item as they are now orphaned
	for (const FName& ChildItemMountPath : DiscoveredItem.ChildItems)
	{
		RemoveDiscoveredItemImpl(ChildItemMountPath, /*bParentIsOrphan*/true);
	}

	if (DiscoveredItem.Type == FDiscoveredItem::EType::Directory)
	{
		FNameBuilder MountPathStr(InMountPath);
		const FStringView MountPathStrView = MountPathStr;
		const uint32 MountPathStrHash = GetTypeHash(MountPathStrView);

		// Deleted paths are no longer relevant for tracking
		AlwaysVisibleFolders.RemoveByHash(MountPathStrHash, MountPathStrView);
		EmptyFolders.RemoveByHash(MountPathStrHash, MountPathStrView);
	}

	// Notify that this item was removed
	// Note: This happens after removing the children so that the notifications are processed in the correct child->parent destruction order
	QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemRemovedUpdate(CreateItemFromDiscovered(InMountPath, DiscoveredItem)));
}

void UContentBrowserFileDataSource::OnPathPopulated(const FName InPath)
{
	OnPathPopulated(FNameBuilder(InPath));
}

void UContentBrowserFileDataSource::OnPathPopulated(const FStringView InPath)
{
	// Recursively un-hide this path, emitting update events for any paths that change state so that the view updates
	if (InPath.Len() > 1)
	{
		// Trim any trailing slash
		FStringView Path = InPath;
		if (Path[Path.Len() - 1] == TEXT('/'))
		{
			Path = Path.Left(Path.Len() - 1);
		}

		// Recurse first as we want parents to be updated before their children
		{
			int32 LastSlashIndex = INDEX_NONE;
			if (Path.FindLastChar(TEXT('/'), LastSlashIndex) && LastSlashIndex > 0)
			{
				OnPathPopulated(Path.Left(LastSlashIndex));
			}
		}

		// Unhide this folder and emit a notification if required
		const uint32 PathHash = GetTypeHash(Path);
		if (EmptyFolders.RemoveByHash(PathHash, Path) > 0)
		{
			// Queue an update event for this path as it may have become visible in the view
			const FName InternalPath = FName(Path);
			if (const FDiscoveredItem* DiscoveredItem = DiscoveredItems.Find(InternalPath))
			{
				QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateFolderItem(InternalPath, DiscoveredItem->DiskPath)));
			}
		}
	}
}

void UContentBrowserFileDataSource::OnAlwaysShowPath(const FName InPath)
{
	OnAlwaysShowPath(FNameBuilder(InPath));
}

void UContentBrowserFileDataSource::OnAlwaysShowPath(const FStringView InPath)
{
	// Recursively force show this path, emitting update events for any paths that change state so that the view updates
	if (InPath.Len() > 1)
	{
		// Trim any trailing slash
		FStringView Path = InPath;
		if (Path[Path.Len() - 1] == TEXT('/'))
		{
			Path.LeftInline(Path.Len() - 1);
		}

		// Recurse first as we want parents to be updated before their children
		{
			int32 LastSlashIndex = INDEX_NONE;
			if (Path.FindLastChar(TEXT('/'), LastSlashIndex) && LastSlashIndex > 0)
			{
				OnAlwaysShowPath(Path.Left(LastSlashIndex));
			}
		}

		// Force show this folder and emit a notification if required
		const uint32 PathHash = GetTypeHash(Path);
		if (!AlwaysVisibleFolders.ContainsByHash(PathHash, Path))
		{
			AlwaysVisibleFolders.Add(FString(Path));

			// Queue an update event for this path as it may have become visible in the view
			const FName InternalPath = FName(Path);
			if (const FDiscoveredItem* DiscoveredItem = DiscoveredItems.Find(InternalPath))
			{
				QueueItemDataUpdate(FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(CreateFolderItem(InternalPath, DiscoveredItem->DiskPath)));
			}
		}
	}
}

void UContentBrowserFileDataSource::OnDirectoryChanged(const TArray<FFileChangeData>& InFileChanges, const FString InFileMountPath, const FString InFileMountDiskPath)
{
	for (const FFileChangeData& FileChange : InFileChanges)
	{
		// The directory watcher doesn't tell us whether the changed item was a directory or a file, so we have to guess...
		// This isn't as simple as checking the item type on disk, as we may be looking at something that has already been removed
		// To this end we assume that anything that has a file extension we care about is a file, and anything that has no extension is a directory
		const FStringView FileExtension = FPathViews::GetExtension(FileChange.Filename);
		const bool bHasExtension = !FileExtension.IsEmpty();
		const bool bKnownFileType = bHasExtension && Config.FindFileActionsForExtension(FileExtension).IsValid();

		if (bKnownFileType)
		{
			const FString FileDiskPath = FPaths::ConvertRelativePathToFull(FileChange.Filename);
			const FString FileMountPath = FileDiskPath.Replace(*InFileMountDiskPath, *InFileMountPath);

			switch (FileChange.Action)
			{
			case FFileChangeData::FCA_Added:
			case FFileChangeData::FCA_Modified:
				{
					// Attempt to sanity check that this is actually a file, and check that the file either exists on disk, or that no directory exists with this name
					const bool bIsFile = FPaths::FileExists(FileChange.Filename) || !FPaths::DirectoryExists(FileChange.Filename);
					AddDiscoveredItem(bIsFile ? FDiscoveredItem::EType::File : FDiscoveredItem::EType::Directory, FileMountPath, FileDiskPath, /*bIsRootPath*/false);

					// Scan the new directory as it may have come from a rename and already contain content
					if (!bIsFile && BackgroundDiscovery)
					{
						BackgroundDiscovery->AddPathToSearch(FileMountPath, FileDiskPath);
					}
				}
				break;

			case FFileChangeData::FCA_Removed:
				RemoveDiscoveredItem(FileMountPath);
				break;

			default:
				break;
			}
		}
		else if (!bHasExtension)
		{
			const FString FileDiskPath = FPaths::ConvertRelativePathToFull(FileChange.Filename);
			const FString FileMountPath = FileDiskPath.Replace(*InFileMountDiskPath, *InFileMountPath);

			switch (FileChange.Action)
			{
			case FFileChangeData::FCA_Added:
				{
					// Attempt to sanity check that this is actually a directory, and check that the directory either exists on disk, or that no file exists with this name
					const bool bIsDirectory = FPaths::DirectoryExists(FileChange.Filename) || !FPaths::FileExists(FileChange.Filename);
					if (bIsDirectory)
					{
						AddDiscoveredItem(FDiscoveredItem::EType::Directory, FileMountPath, FileDiskPath, /*bIsRootPath*/false);

						// Scan the new directory as it may have come from a rename and already contain content
						if (BackgroundDiscovery)
						{
							BackgroundDiscovery->AddPathToSearch(FileMountPath, FileDiskPath);
						}
					}
				}
				break;

			case FFileChangeData::FCA_Removed:
				RemoveDiscoveredItem(FileMountPath);
				break;

			default:
				break;
			}
		}
	}
}

void UContentBrowserFileDataSource::PopulateAddNewContextMenu(UToolMenu* InMenu)
{
	const UContentBrowserDataMenuContext_AddNewMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_AddNewMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_AddNewMenu was missing!"));

	// Extract the internal file paths that belong to this data source from the full list of selected paths given in the context
	TArray<FName> SelectedFilePaths;
	for (const FName& SelectedPath : ContextObject->SelectedPaths)
	{
		FName InternalPath;
		if (TryConvertVirtualPathToInternal(SelectedPath, InternalPath) && IsKnownFileMount(InternalPath))
		{
			SelectedFilePaths.Add(InternalPath);
		}
	}

	// Only add the file items if we have a file path selected
	if (SelectedFilePaths.Num() > 0)
	{
		const FCanExecuteAction CanExecuteAssetActionsDelegate = FCanExecuteAction::CreateLambda([NumSelectedFilePaths = SelectedFilePaths.Num()]()
		{
			// We can execute file actions when we only have a single file path selected
			return NumSelectedFilePaths == 1;
		});

		const FName FirstSelectedPath = (SelectedFilePaths.Num() > 0) ? SelectedFilePaths[0] : FName();
		FString FirstSelectedDiskPath;
		IsKnownFileMount(FirstSelectedPath, &FirstSelectedDiskPath);

		// Create
		{
			FToolMenuSection& Section = InMenu->AddSection("ContentBrowserCreateFile", LOCTEXT("CreateFileMenuHeading", "Create File"));

			Config.EnumerateFileActions([this, ContextObject, &Section, &CanExecuteAssetActionsDelegate, &FirstSelectedPath, &FirstSelectedDiskPath](TSharedRef<const ContentBrowserFileData::FFileActions> InFileActions)
			{
				if (!InFileActions->CanCreate.IsBound() || InFileActions->CanCreate.Execute(FirstSelectedPath, FirstSelectedDiskPath, nullptr))
				{
					const FName MenuItemName = *FString::Printf(TEXT("CreateFile_%s"), *InFileActions->TypeName.GetAssetName().ToString());
					const FText MenuItemTitle = InFileActions->TypeShortDescription.IsEmpty()
						? FText::Format(LOCTEXT("CreateFileWithName", "Create {0} file"), InFileActions->TypeDisplayName)
						: FText::Format(LOCTEXT("CreateFileWithDesc", "Create {0}"), InFileActions->TypeShortDescription);

					Section.AddMenuEntry(
						MenuItemName,
						MenuItemTitle,
						InFileActions->TypeFullDescription,
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateUObject(this, &UContentBrowserFileDataSource::OnNewFileRequested, FirstSelectedPath, FirstSelectedDiskPath, InFileActions, ContextObject->OnBeginItemCreation),
							CanExecuteAssetActionsDelegate
							)
						);
				}

				return true;
			});
		}
	}
}

void UContentBrowserFileDataSource::OnNewFileRequested(const FName InDestFolderPath, const FString InDestFolder, TSharedRef<const ContentBrowserFileData::FFileActions> InFileActions, UContentBrowserDataMenuContext_AddNewMenu::FOnBeginItemCreation InOnBeginItemCreation)
{
	if (!ensure(InOnBeginItemCreation.IsBound()))
	{
		return;
	}

	FStructOnScope CreationConfig;
	FString SuggestedFilename;
	if (InFileActions->ConfigureCreation.IsBound())
	{
		if (!InFileActions->ConfigureCreation.Execute(SuggestedFilename, CreationConfig))
		{
			return;
		}
	}

	if (SuggestedFilename.IsEmpty())
	{
		SuggestedFilename = InFileActions->DefaultNewFileName.IsEmpty() ? FString::Printf(TEXT("New%sFile"), *InFileActions->TypeName.GetAssetName().ToString()) : InFileActions->DefaultNewFileName;
	}
	SuggestedFilename += TEXT(".");
	SuggestedFilename += InFileActions->TypeExtension;

	FString NewFilename = InDestFolder / SuggestedFilename;
	ContentBrowserFileData::MakeUniqueFilename(NewFilename);

	FString NewFilePath = NewFilename;
	NewFilePath.ReplaceInline(*InDestFolder, *InDestFolderPath.ToString());

	FName NewInternalFilePath = *NewFilePath;

	FName VirtualizedPath;
	TryConvertInternalPathToVirtual(NewInternalFilePath, VirtualizedPath);

	const FString NewFileItemName = FPaths::GetBaseFilename(NewFilename);

	FContentBrowserItemData NewItemData(
		this,
		EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Category_Misc
			| EContentBrowserItemFlags::Temporary_Creation,
		VirtualizedPath,
		*NewFileItemName,
		FText::AsCultureInvariant(NewFileItemName),
		MakeShared<FContentBrowserFileItemDataPayload_Creation>(
			NewInternalFilePath, NewFilename, InFileActions, MoveTemp(CreationConfig)),
		{ NewInternalFilePath });

	InOnBeginItemCreation.Execute(FContentBrowserItemDataTemporaryContext(
		MoveTemp(NewItemData),
		FContentBrowserItemDataTemporaryContext::FOnValidateItem::CreateUObject(
			this, &UContentBrowserFileDataSource::OnValidateItemName),
		FContentBrowserItemDataTemporaryContext::FOnFinalizeItem::CreateUObject(
			this, &UContentBrowserFileDataSource::OnFinalizeCreateFile)));
}

bool UContentBrowserFileDataSource::OnValidateItemName(const FContentBrowserItemData& InItem, const FString& InProposedName, FText* OutErrorMsg)
{
	return CanRenameItem(InItem, &InProposedName, nullptr, OutErrorMsg);
}

FContentBrowserItemData UContentBrowserFileDataSource::OnFinalizeCreateFolder(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg)
{
	checkf(InItemData.GetOwnerDataSource() == this, TEXT("OnFinalizeCreateFolder was bound to an instance from the wrong data source!"));
	checkf(EnumHasAllFlags(InItemData.GetItemFlags(), EContentBrowserItemFlags::Type_Folder | EContentBrowserItemFlags::Temporary_Creation), TEXT("OnFinalizeCreateFolder called for an instance with the incorrect type flags!"));

	// Committed creation
	if (TSharedPtr<const FContentBrowserFolderItemDataPayload> FolderPayload = GetFolderItemPayload(InItemData))
	{
		auto CreateDirectory = [&FolderPayload](const FName InNewInternalPath, const FString& InNewInternalDiskPath)
		{
			if (TSharedPtr<const ContentBrowserFileData::FDirectoryActions> DirectoryActions = FolderPayload->GetDirectoryActions())
			{
				if (DirectoryActions->Create.IsBound())
				{
					return DirectoryActions->Create.Execute(InNewInternalPath, InNewInternalDiskPath, FStructOnScope());
				}
			}

			return IFileManager::Get().MakeDirectory(*InNewInternalDiskPath, true);
		};

		const FName NewInternalPath = *(FPaths::GetPath(FolderPayload->GetInternalPath().ToString()) / InProposedName);
		const FString NewPathOnDisk = FPaths::GetPath(FolderPayload->GetFilename()) / InProposedName;
		if (CreateDirectory(NewInternalPath, NewPathOnDisk))
		{
			return CreateFolderItem(NewInternalPath, NewPathOnDisk);
		}
	}

	ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_FailedToCreateFolder", "Failed to create folder"));
	return FContentBrowserItemData();
}

FContentBrowserItemData UContentBrowserFileDataSource::OnFinalizeCreateFile(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg)
{
	checkf(InItemData.GetOwnerDataSource() == this, TEXT("OnFinalizeCreateFile was bound to an instance from the wrong data source!"));
	checkf(EnumHasAllFlags(InItemData.GetItemFlags(), EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Temporary_Creation), TEXT("OnFinalizeCreateFile called for an instance with the incorrect type flags!"));

	// Committed creation
	if (TSharedPtr<const FContentBrowserFileItemDataPayload_Creation> CreateFilePayload = StaticCastSharedPtr<const FContentBrowserFileItemDataPayload_Creation>(GetFileItemPayload(InItemData)))
	{
		auto CreateFile = [&CreateFilePayload](const FName InNewInternalPath, const FString& InNewInternalDiskPath)
		{
			if (TSharedPtr<const ContentBrowserFileData::FFileActions> FileActions = CreateFilePayload->GetFileActions())
			{
				if (FileActions->Create.IsBound())
				{
					return FileActions->Create.Execute(InNewInternalPath, InNewInternalDiskPath, CreateFilePayload->GetCreationConfig());
				}
			}

			return FFileHelper::SaveStringToFile(TEXT(""), *InNewInternalDiskPath);
		};

		const FString Extension = FPaths::GetExtension(CreateFilePayload->GetFilename(), /*bIncludeDot*/true);
		const FName NewInternalPath = *(FPaths::GetPath(CreateFilePayload->GetInternalPath().ToString()) / InProposedName + Extension);
		const FString NewPathOnDisk = FPaths::GetPath(CreateFilePayload->GetFilename()) / InProposedName + Extension;
		if (CreateFile(NewInternalPath, NewPathOnDisk))
		{
			return CreateFileItem(NewInternalPath, NewPathOnDisk);
		}
	}

	ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_FailedToCreateFile", "Failed to create file"));
	return FContentBrowserItemData();
}

FContentBrowserItemData UContentBrowserFileDataSource::OnFinalizeDuplicateFile(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg)
{
	checkf(InItemData.GetOwnerDataSource() == this, TEXT("OnFinalizeDuplicateFile was bound to an instance from the wrong data source!"));
	checkf(EnumHasAllFlags(InItemData.GetItemFlags(), EContentBrowserItemFlags::Type_File | EContentBrowserItemFlags::Temporary_Duplication), TEXT("OnFinalizeDuplicateFile called for an instance with the incorrect type flags!"));

	// Committed duplication
	{
		TSharedPtr<const FContentBrowserFileItemDataPayload_Duplication> DuplicationContext = StaticCastSharedPtr<const FContentBrowserFileItemDataPayload_Duplication>(InItemData.GetPayload());
	
		const FString Extension = FPaths::GetExtension(DuplicationContext->GetFilename(), /*bIncludeDot*/true);
		const FString NewFilename = FPaths::GetPath(DuplicationContext->GetFilename()) / InProposedName + Extension;

		if (IFileManager::Get().Copy(*NewFilename, *DuplicationContext->GetSourceFilename(), /*bReplace*/false) == COPY_OK)
		{
			USourceControlHelpers::MarkFileForAdd(NewFilename, /*bSilent*/true);

			const FString NewInternalPath = FPaths::GetPath(DuplicationContext->GetInternalPath().ToString()) / InProposedName + Extension;
			return CreateFileItem(*NewInternalPath, NewFilename);
		}
	}
	
	ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, LOCTEXT("Error_FailedToCreateFile", "Failed to create file"));
	return FContentBrowserItemData();
}

#undef LOCTEXT_NAMESPACE

