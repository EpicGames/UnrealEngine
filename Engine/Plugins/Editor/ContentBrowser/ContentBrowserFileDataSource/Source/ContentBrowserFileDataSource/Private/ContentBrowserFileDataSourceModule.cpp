// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserDataMenuContexts.h"
#include "FileSourceControlContextMenu.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"

class FContentBrowserFileDataSourceModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
		SCCContextMenu = MakeShared<FFileSourceControlContextMenu>();

		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu"))
		{
			Menu->AddDynamicSection(TEXT("DynamicSection_FileDataSource_SCC"), FNewToolMenuDelegate::CreateLambda([WeakSCCContextMenu = TWeakPtr<FFileSourceControlContextMenu>(SCCContextMenu)](UToolMenu* InMenu)
			{
				if (TSharedPtr<FFileSourceControlContextMenu> SCCContextMenuPin = WeakSCCContextMenu.Pin())
				{
					FToolMenuSection& Section = InMenu->FindOrAddSection("AssetContextReferences");
					SCCContextMenuPin->MakeContextMenu(Section, InMenu->FindContext<UContentBrowserDataMenuContext_FileMenu>(), InMenu->FindContext<UContentBrowserDataMenuContext_FolderMenu>());
				}
			}));
		}

		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.FolderContextMenu"))
		{
			Menu->AddDynamicSection(TEXT("DynamicSection_FileDataSource_SCC"), FNewToolMenuDelegate::CreateLambda([WeakSCCContextMenu = TWeakPtr<FFileSourceControlContextMenu>(SCCContextMenu)](UToolMenu* InMenu)
			{
				if (TSharedPtr<FFileSourceControlContextMenu> SCCContextMenuPin = WeakSCCContextMenu.Pin())
				{
					FToolMenuSection& Section = InMenu->FindOrAddSection("PathContextBulkOperations");
					SCCContextMenuPin->MakeContextMenu(Section, InMenu->FindContext<UContentBrowserDataMenuContext_FileMenu>(), InMenu->FindContext<UContentBrowserDataMenuContext_FolderMenu>());
				}
			}));
		}
	}

	virtual void ShutdownModule() override
	{
		SCCContextMenu.Reset();
	}

private:
	TSharedPtr<FFileSourceControlContextMenu> SCCContextMenu;
};

IMPLEMENT_MODULE(FContentBrowserFileDataSourceModule, ContentBrowserFileDataSource);
