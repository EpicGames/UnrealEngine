// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileSourceControlContextMenu.h"

#include "Async/Async.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserFileDataCore.h"
#include "ContentBrowserFileDataSource.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "ISourceControlModule.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "SourceControlWindows.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "UncontrolledChangelistsModule.h"

#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "FileSourceControlContextMenu"

namespace UE::FileSourceControlContextMenu::Private
{
	DECLARE_DELEGATE_RetVal(bool, FIsAsyncProcessingActive);

	FNewToolMenuCustomWidget MakeCustomWidgetDelegate(const TAttribute<FText>& Label, const TAttribute<FSlateIcon>& Icon, const FIsAsyncProcessingActive& IsAsyncProcessingActive)
	{
		return FNewToolMenuCustomWidget::CreateLambda([Label, Icon, IsAsyncProcessingActive](const FToolMenuContext& InContext, const FToolMenuCustomWidgetContext& WidgetContext)
			{
				return SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(2, 0, 0, 0))
					.AutoWidth()
					[
						SNew(SImage)
						.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
						.Image(TAttribute<const FSlateBrush*>::CreateLambda([Icon]()
						{
							return Icon.IsSet() ? Icon.Get().GetIcon() : nullptr;
						}))
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(8, 0, 0, 0))
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(Label)
						.TextStyle(WidgetContext.StyleSet, ISlateStyle::Join(WidgetContext.StyleName, ".Label"))
					]
					+SHorizontalBox::Slot()
					.Padding(FMargin(8, 0, 0, 0))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SThrobber)
						.PieceImage(WidgetContext.StyleSet->GetBrush("Throbber.CircleChunk"))
						.Visibility_Lambda([IsAsyncProcessingActive]()
						{
							if (IsAsyncProcessingActive.Execute())
							{
								return EVisibility::Visible;
							}
							return EVisibility::Hidden;
						})
					];
			});
	};


	FToolMenuEntry& AddAsyncMenuEntry(
		FToolMenuSection& Section,
		FName Name,
		const TAttribute<FText>& Label,
		const TAttribute<FText>& ToolTip,
		const TAttribute<FSlateIcon>& Icon,
		const FToolUIActionChoice& Action,
		const FIsAsyncProcessingActive& IsAsyncProcessingActive
	)
	{
		FToolMenuEntry MenuEntry = FToolMenuEntry::InitMenuEntry(
			Name,
			Label,
			ToolTip,
			Icon,
			Action
		);

		if (IsAsyncProcessingActive.IsBound())
		{
			MenuEntry.MakeCustomWidget = MakeCustomWidgetDelegate(Label, Icon, IsAsyncProcessingActive);
		}
		return Section.AddEntry(MenuEntry);
	}
}

class FFileSourceControlContextMenuState : public TSharedFromThis<FFileSourceControlContextMenuState>
{
public:
	void Initialize(FToolMenuSection& InSection, const UContentBrowserDataMenuContext_FileMenu* InFileMenuContext, const UContentBrowserDataMenuContext_FolderMenu* InFolderMenuContext);

	bool IsValid() const;

	void Close();

private:
	bool AddSourceControlMenuOptions(FToolMenuSection& InSection);
	void FillSourceControlSubMenu(UToolMenu* Menu);

	FExecuteAction ExecutionCheck(FExecuteAction&& InAction) const;

	void ExecuteSCCCheckOut() const;
	void ExecuteSCCSyncAndCheckOut() const;
	void ExecuteSCCMakeWritable() const;
	void ExecuteSCCOpenForAdd() const;
	void ExecuteSCCCheckIn() const;
	void ExecuteSCCHistory() const;
	void ExecuteSCCDiffAgainstDepot() const;
	void ExecuteSCCRevert() const;
	void ExecuteSCCSync() const;
	void ExecuteSCCRefresh() const;

	bool CanExecuteSCCCheckOut() const;
	bool CanExecuteSCCSyncAndCheckOut() const;
	bool CanExecuteSCCMakeWritable() const;
	bool CanExecuteSCCOpenForAdd() const;
	bool CanExecuteSCCCheckIn() const;
	bool CanExecuteSCCHistory() const;
	bool CanExecuteSCCRevert() const;
	bool CanExecuteSCCSync() const;
	bool CanExecuteSCCDiffAgainstDepot() const;

private:

	enum class EAsyncState : uint8
	{
		None,
		TryCacheCanExecute,
		SCCUpdate,
		Idle,
		Error,
		Cancelled,
	};

	bool IsActionEnabled(bool bEnabled) const;

	bool IsStillScanning(bool bEnabled) const;

	bool TrySetAsyncState(EAsyncState FromState, EAsyncState ToState);

	/** Initializes some variable used to in "CanExecute" checks that won't change at runtime or are too expensive to check every frame. */
	void NextAsyncState();

	/** Callback from a perforce update call. */
	void CacheCanExecuteVarsSCCHandler(const FSourceControlOperationRef& InOperation, ECommandResult::Type Result);

	/** Try using perforce cache to update state */
	void TryCacheCanExecuteVars(const TArray<FString>& InSelectedPaths, TArray<FString>* OptionalPathsWithUnknownState);

private:

	TArray<FString> SelectedFiles;
	TArray<FString> PathsWithUnknownState;
	TArray<FString> CheckedOutUsers;
	FText CheckedOutUsersText;

	TSharedPtr<class ISourceControlOperation, ESPMode::ThreadSafe> SCCOperation;
	std::atomic<EAsyncState> AsyncState = EAsyncState::None;

	// If folders are present in the selection then we adjust some behaviors to account for the potentially large amount of files to scan
	bool bContainsFolders = false;

	/** Cached CanExecute vars */
	bool bCanExecuteSCCCheckOut = false;
	bool bCanExecuteSCCSyncAndCheckOut = false;
	bool bCanExecuteSCCMakeWritable = false;
	bool bCanExecuteSCCOpenForAdd = false;
	bool bCanExecuteSCCCheckIn = false;
	bool bCanExecuteSCCHistory = false;
	bool bCanExecuteSCCRevert = false;
	bool bCanExecuteSCCSync = false;
};

FFileSourceControlContextMenu::~FFileSourceControlContextMenu()
{
	if (InnerState)
	{
		InnerState->Close();
	}
}

void FFileSourceControlContextMenu::MakeContextMenu(FToolMenuSection& InSection, const UContentBrowserDataMenuContext_FileMenu* InFileMenuContext, const UContentBrowserDataMenuContext_FolderMenu* InFolderMenuContext)
{
	if (InnerState)
	{
		InnerState->Close();
	}

	InnerState = MakeShared<FFileSourceControlContextMenuState>();
	InnerState->Initialize(InSection, InFileMenuContext, InFolderMenuContext);
}

void FFileSourceControlContextMenuState::Initialize(FToolMenuSection& InSection, const UContentBrowserDataMenuContext_FileMenu* InFileMenuContext, const UContentBrowserDataMenuContext_FolderMenu* InFolderMenuContext)
{
	if (InFileMenuContext && InFileMenuContext->bCanBeModified)
	{
		for (const FContentBrowserItem& SelectedItem : InFileMenuContext->SelectedItems)
		{
			if (SelectedItem.IsFile())
			{
				for (const FContentBrowserItemData& SelectedItemData : SelectedItem.GetInternalItems())
				{
					if (Cast<UContentBrowserFileDataSource>(SelectedItemData.GetOwnerDataSource()))
					{
						FString ItemDiskPath;
						if (ContentBrowserFileData::GetItemPhysicalPath(SelectedItemData.GetOwnerDataSource(), SelectedItemData, ItemDiskPath))
						{
							SelectedFiles.Add(MoveTemp(ItemDiskPath));
						}
					}
				}
			}
		}
	}

	// TODO: Cannot support folders while both the asset and file data sources implement this menu separately - need to unify this logic between them!
	/*
	if (InFolderMenuContext && InFolderMenuContext->bCanBeModified)
	{
		for (const FContentBrowserItem& SelectedItem : InFolderMenuContext->SelectedItems)
		{
			if (SelectedItem.IsFolder())
			{
				for (const FContentBrowserItemData& SelectedItemData : SelectedItem.GetInternalItems())
				{
					if (Cast<UContentBrowserFileDataSource>(SelectedItemData.GetOwnerDataSource()))
					{
						FContentBrowserDataFilter Filter;
						Filter.bRecursivePaths = true;
						Filter.ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeFiles;

						FContentBrowserDataCompiledFilter CompiledFilter;
						SelectedItemData.GetOwnerDataSource()->CompileFilter(SelectedItemData.GetVirtualPath(), Filter, CompiledFilter);

						SelectedItemData.GetOwnerDataSource()->EnumerateItemsMatchingFilter(CompiledFilter, [this, &SelectedItemData](FContentBrowserItemData&& ItemData)
						{
							if (ItemData.IsFile())
							{
								FString ItemDiskPath;
								if (ContentBrowserFileData::GetItemPhysicalPath(SelectedItemData.GetOwnerDataSource(), ItemData, ItemDiskPath))
								{
									SelectedFiles.Add(MoveTemp(ItemDiskPath));
								}
							}
							return true;
						});
					}
				}
			}
		}

		bContainsFolders = true;
	}
	*/

	if (IsValid())
	{
		AddSourceControlMenuOptions(InSection);
	}
}

bool FFileSourceControlContextMenuState::IsValid() const
{
	return SelectedFiles.Num() > 0;
}

void FFileSourceControlContextMenuState::Close()
{
	AsyncState.store(EAsyncState::Cancelled);

	if (SCCOperation.IsValid())
	{
		ISourceControlModule::Get().GetProvider().CancelOperation(SCCOperation.ToSharedRef());
	}
}

bool FFileSourceControlContextMenuState::AddSourceControlMenuOptions(FToolMenuSection& InSection)
{
	if (ISourceControlModule::Get().IsEnabled())
	{
		if (ISourceControlModule::Get().GetProvider().IsAvailable())
		{
			// Start work to cache execution state, this can take a while if SCC cache is cold
			NextAsyncState();

			// SCC sub menu
			InSection.AddSubMenu(
				"SourceControlSubMenu",
				LOCTEXT("SourceControlSubMenuLabel", "Revision Control"),
				LOCTEXT("SourceControlSubMenuToolTip", "Revision Control actions."),
				FNewToolMenuDelegate::CreateSP(this, &FFileSourceControlContextMenuState::FillSourceControlSubMenu),
				false,
				FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Icon", FRevisionControlStyleManager::GetStyleSetName() , "RevisionControl.Icon.ConnectedBadge")
			);
		}
		else
		{
			InSection.AddSubMenu(
				"SourceControlSubMenu",
				LOCTEXT("SourceControlSubMenuUnavailableLabel", "Revision Control (Unavailable)"),
				LOCTEXT("SourceControlSubMenuUnavailableToolTip", "Revision Control servers are currently unavailable, check your network connection."),
				FNewToolMenuDelegate(),
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateLambda([]() { return false; })
				),
				EUserInterfaceActionType::Button,
				false,
				FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Icon", FRevisionControlStyleManager::GetStyleSetName() , "RevisionControl.Icon.WarningBadge")
			);
		}
	}
	else
	{
		InSection.AddMenuEntry(
			"SCCConnectToSourceControl",
			LOCTEXT("SCCConnectToSourceControl", "Connect to Revision Control..."),
			LOCTEXT("SCCConnectToSourceControlTooltip", "Connect to a revision control system for tracking changes to your content and levels."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Icon"),
			FUIAction(
				FExecuteAction::CreateLambda([]() { ISourceControlModule::Get().ShowLoginDialog(FSourceControlLoginClosed(), ELoginWindowMode::Modeless); }),
				FCanExecuteAction()
			)
		);
	}

	return true;
}

void FFileSourceControlContextMenuState::FillSourceControlSubMenu(UToolMenu* Menu)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	FToolMenuSection& Section = Menu->AddSection("FileSourceControlActions", LOCTEXT("FileSourceControlActionsMenuHeading", "Revision Control"));

	using namespace UE::FileSourceControlContextMenu::Private;

	const bool bUsesCheckout = SourceControlProvider.UsesCheckout();
	const bool bUsesFileRevisions = SourceControlProvider.UsesFileRevisions();
	const bool bUsesSnapshots = SourceControlProvider.UsesSnapshots();
	const bool bUsesReadOnly = SourceControlProvider.UsesLocalReadOnlyState();
	const bool bUsesDiffAgainstDepot = SourceControlProvider.AllowsDiffAgainstDepot();

	if (bUsesFileRevisions)
	{
		AddAsyncMenuEntry(Section,
			"SCCSync",
			LOCTEXT("SCCSync", "Sync"),
			LOCTEXT("SCCSyncTooltip", "Updates the selected files to the latest version in revision control."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Sync"),
			FUIAction(
				ExecutionCheck(FExecuteAction::CreateSP(this, &FFileSourceControlContextMenuState::ExecuteSCCSync)),
				FCanExecuteAction::CreateLambda([this]() { return IsActionEnabled(CanExecuteSCCSync()); })
			),
			FIsAsyncProcessingActive::CreateLambda([this]() { return IsStillScanning(CanExecuteSCCSync()); })
		);
	}

	if (bUsesCheckout)
	{
		AddAsyncMenuEntry(Section,
			"SCCCheckOut",
			LOCTEXT("SCCCheckOut", "Check Out"),
			TAttribute<FText>::CreateLambda([this]()
			{
				if (IsStillScanning(CanExecuteSCCCheckOut()) || CheckedOutUsers.Num() == 0)
				{
					return LOCTEXT("SCCCheckOutTooltip", "Check out the selected files from revision control.");
				}

				return FText::Format(LOCTEXT("SCCPartialCheckOut", "Checks out the selected files from revision control that are not currently locked.\n\nLocked Files:\n{0}"), CheckedOutUsersText);
			}),
			TAttribute<FSlateIcon>::CreateLambda([this]()
			{
				return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), CheckedOutUsers.Num() == 0 ? "RevisionControl.Actions.CheckOut" : "RevisionControl.Locked");
			}),
			FUIAction(
				ExecutionCheck(FExecuteAction::CreateSP(this, &FFileSourceControlContextMenuState::ExecuteSCCCheckOut)),
				FCanExecuteAction::CreateLambda([this]() { return IsActionEnabled(CanExecuteSCCCheckOut()); })
			),
			FIsAsyncProcessingActive::CreateLambda([this]() { return IsStillScanning(CanExecuteSCCCheckOut()); })
			);

		AddAsyncMenuEntry(Section,
			"SCCSyncAndCheckOut",
			LOCTEXT("SCCSyncAndCheckOut", "Sync And Check Out"),
			TAttribute<FText>::CreateLambda([this]()
			{
				if (IsStillScanning(CanExecuteSCCSyncAndCheckOut()) || CheckedOutUsers.Num() == 0)
				{
					return LOCTEXT("SCCSyncAndCheckOutTooltip", "Sync to latest and Check out the selected files from revision control.");
				}

				return FText::Format(LOCTEXT("SCCPartialSyncAndCheckOut", "Sync to latest and Checks out the selected files from revision control that are not currently locked.\n\nLocked Files:\n{0}"), CheckedOutUsersText);
			}),
			TAttribute<FSlateIcon>::CreateLambda([this]()
			{
				return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), CheckedOutUsers.Num() == 0 ? "RevisionControl.Actions.SyncAndCheckOut" : "RevisionControl.Locked");
			}),
			FUIAction(
				ExecutionCheck(FExecuteAction::CreateSP(this, &FFileSourceControlContextMenuState::ExecuteSCCSyncAndCheckOut)),
				FCanExecuteAction::CreateLambda([this]() { return IsActionEnabled(CanExecuteSCCSyncAndCheckOut()); })
			),
			FIsAsyncProcessingActive::CreateLambda([this]() { return IsStillScanning(CanExecuteSCCSyncAndCheckOut()); })
			);
	}

	if (bUsesReadOnly)
	{
		AddAsyncMenuEntry(Section,
			"SCCMakeWritable",
			LOCTEXT("SCCMakeWritable", "Make Writable"),
			LOCTEXT("SCCMakeWritableTooltip", "Remove read-only flag from all selected files."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.MakeWritable"),
			FUIAction(
				ExecutionCheck(FExecuteAction::CreateSP(this, &FFileSourceControlContextMenuState::ExecuteSCCMakeWritable)),
				FCanExecuteAction::CreateLambda([this]() { return IsActionEnabled(CanExecuteSCCMakeWritable()); })
			),
			FIsAsyncProcessingActive::CreateLambda([this]() { return IsStillScanning(CanExecuteSCCMakeWritable()); })
		);
	}

	AddAsyncMenuEntry(Section,
		"SCCOpenForAdd",
		LOCTEXT("SCCOpenForAdd", "Mark For Add"),
		LOCTEXT("SCCOpenForAddTooltip", "Adds the selected files to revision control."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Add"),
		FUIAction(
			ExecutionCheck(FExecuteAction::CreateSP(this, &FFileSourceControlContextMenuState::ExecuteSCCOpenForAdd)),
			FCanExecuteAction::CreateLambda([this]() { return IsActionEnabled(CanExecuteSCCOpenForAdd()); })
		),
		FIsAsyncProcessingActive::CreateLambda([this]() { return IsStillScanning(CanExecuteSCCOpenForAdd()); })
	);

	if (!bUsesSnapshots)
	{
		AddAsyncMenuEntry(Section,
			"SCCCheckIn",
			LOCTEXT("SCCCheckIn", "Check In"),
			LOCTEXT("SCCCheckInTooltip", "Checks the selected files into revision control."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Submit"),
			FUIAction(
				ExecutionCheck(FExecuteAction::CreateSP(this, &FFileSourceControlContextMenuState::ExecuteSCCCheckIn)),
				FCanExecuteAction::CreateLambda([this]() { return IsActionEnabled(CanExecuteSCCCheckIn()); })
			),
			FIsAsyncProcessingActive::CreateLambda([this]() { return IsStillScanning(CanExecuteSCCCheckIn()); })
		);
	}

	AddAsyncMenuEntry(Section,
		"SCCHistory",
		LOCTEXT("SCCHistory", "History"),
		LOCTEXT("SCCHistoryTooltip", "Displays the history of the selected files in revision control."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.History"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FFileSourceControlContextMenuState::ExecuteSCCHistory),
			FCanExecuteAction::CreateLambda([this]() { return IsActionEnabled(CanExecuteSCCHistory()); })
		),
		FIsAsyncProcessingActive::CreateLambda([this]() { return IsStillScanning(CanExecuteSCCHistory()); })
	);

	if (bUsesDiffAgainstDepot)
	{
		AddAsyncMenuEntry(Section,
			"SCCDiffAgainstDepot",
			LOCTEXT("SCCDiffAgainstDepot", "Diff Against Depot"),
			LOCTEXT("SCCDiffAgainstDepotTooltip", "Look at differences between the local and remote version of the selected files."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Diff"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FFileSourceControlContextMenuState::ExecuteSCCDiffAgainstDepot),
				FCanExecuteAction::CreateLambda([this]() { return IsActionEnabled(CanExecuteSCCDiffAgainstDepot()); })
			),
			FIsAsyncProcessingActive::CreateLambda([this]() { return IsStillScanning(CanExecuteSCCDiffAgainstDepot()); })
		);
	}

	AddAsyncMenuEntry(Section,
		"SCCRevert",
		LOCTEXT("SCCRevert", "Revert"),
		LOCTEXT("SCCRevertTooltip", "Reverts the selected files to their original state from revision control."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Revert"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FFileSourceControlContextMenuState::ExecuteSCCRevert),
			FCanExecuteAction::CreateLambda([this]() { return IsActionEnabled(CanExecuteSCCRevert()); })
		),
		FIsAsyncProcessingActive::CreateLambda([this]() { return IsStillScanning(CanExecuteSCCRevert()); })
	);

	Section.AddMenuEntry(
		"SCCRefresh",
		LOCTEXT("SCCRefresh", "Refresh"),
		LOCTEXT("SCCRefreshTooltip", "Updates the revision control status of the file."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Refresh"),
		FUIAction(
			ExecutionCheck(FExecuteAction::CreateSP(this, &FFileSourceControlContextMenuState::ExecuteSCCRefresh)),
			FCanExecuteAction()
		)
	);
}

FExecuteAction FFileSourceControlContextMenuState::ExecutionCheck(FExecuteAction&& InAction) const
{
	return FExecuteAction::CreateLambda([this, Action = MoveTemp(InAction)]()
	{
		if (SelectedFiles.Num() > 10) // Todo: Make this into a user option, or different values per operation type
		{
			FText Message = FText::Format(LOCTEXT("SCCLargeOperationWarningMessage", "You are about to perform this operation on a large amount of files ({0}), are you sure you want to continue?\n\nUnreal Editor may become unresponsive."), SelectedFiles.Num());
			FText Title = LOCTEXT("SCCLargeOperationWarningTitle", "Continue Operation?");
			EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, Message, Title);

			if (Result != EAppReturnType::Yes)
			{
				return;
			}
		}

		FScopedSlowTask SlowTask(0, LOCTEXT("SCCOperationSlowTaskLabel", "Performing Revision Control Operation"));
		SlowTask.MakeDialogDelayed(0.25f);
		Action.Execute();
	});
}

void FFileSourceControlContextMenuState::ExecuteSCCCheckOut() const
{
	USourceControlHelpers::CheckOutFiles(SelectedFiles, /*bSilent*/true);
}

void FFileSourceControlContextMenuState::ExecuteSCCSyncAndCheckOut() const
{
	ExecuteSCCSync();
	ExecuteSCCCheckOut();
}

void FFileSourceControlContextMenuState::ExecuteSCCMakeWritable() const
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FUncontrolledChangelistsModule& UncontrolledChangelistsModule = FUncontrolledChangelistsModule::Get();

	for (const FString& SelectedFile : SelectedFiles)
	{
		PlatformFile.SetReadOnly(*SelectedFile, false);
		UncontrolledChangelistsModule.OnMakeWritable(SelectedFile);
	}
}

void FFileSourceControlContextMenuState::ExecuteSCCOpenForAdd() const
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	TArray<FString> FilesToAdd;
	for (const FString& SelectedFile : SelectedFiles)
	{
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(SelectedFile, EStateCacheUsage::Use);
		if (SourceControlState.IsValid() && !SourceControlState->IsSourceControlled())
		{
			if (FPaths::FileExists(SelectedFile))
			{
				FilesToAdd.Add(SelectedFile);
			}
		}
	}

	if (FilesToAdd.Num() > 0)
	{
		USourceControlHelpers::MarkFilesForAdd(FilesToAdd, /*bSilent*/true);
	}
}

void FFileSourceControlContextMenuState::ExecuteSCCCheckIn() const
{
	FCheckinResultInfo ResultInfo;
	FSourceControlWindows::PromptForCheckin(ResultInfo, TArray<FString>(), TArray<FString>(), SelectedFiles, /*bUseSourceControlStateCache*/true);

	if (ResultInfo.Result == ECommandResult::Failed)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SCC_Checkin_Failed", "Check-in failed."));
	}
}

void FFileSourceControlContextMenuState::ExecuteSCCHistory() const
{
	FSourceControlWindows::DisplayRevisionHistory(SelectedFiles);
}

void FFileSourceControlContextMenuState::ExecuteSCCDiffAgainstDepot() const
{
	FSourceControlWindows::DiffAgainstWorkspace(SelectedFiles[0]);
}

void FFileSourceControlContextMenuState::ExecuteSCCRevert() const
{
	USourceControlHelpers::RevertFiles(SelectedFiles, /*bSilent*/true);
}

void FFileSourceControlContextMenuState::ExecuteSCCSync() const
{
	if (USourceControlHelpers::SyncFiles(SelectedFiles, /*bSilent*/true))
	{
		ISourceControlModule::Get().GetProvider().Execute(ISourceControlOperation::Create<FUpdateStatus>(), SelectedFiles);
	}
}

void FFileSourceControlContextMenuState::ExecuteSCCRefresh() const
{
	TSharedPtr<FUpdateStatus, ESPMode::ThreadSafe> Operation = ISourceControlOperation::Create<FUpdateStatus>();
	Operation->SetForceUpdate(true);

	ISourceControlModule::Get().GetProvider().Execute(Operation.ToSharedRef(), SelectedFiles, EConcurrency::Asynchronous);
}

bool FFileSourceControlContextMenuState::CanExecuteSCCCheckOut() const
{
	return bCanExecuteSCCCheckOut;
}

bool FFileSourceControlContextMenuState::CanExecuteSCCSyncAndCheckOut() const
{
	return bCanExecuteSCCSyncAndCheckOut;
}

bool FFileSourceControlContextMenuState::CanExecuteSCCMakeWritable() const
{
	return bCanExecuteSCCMakeWritable;
}

bool FFileSourceControlContextMenuState::CanExecuteSCCOpenForAdd() const
{
	return bCanExecuteSCCOpenForAdd;
}

bool FFileSourceControlContextMenuState::CanExecuteSCCCheckIn() const
{
	return bCanExecuteSCCCheckIn;
}

bool FFileSourceControlContextMenuState::CanExecuteSCCHistory() const
{
	return bCanExecuteSCCHistory;
}

bool FFileSourceControlContextMenuState::CanExecuteSCCDiffAgainstDepot() const
{
	return bCanExecuteSCCHistory;
}

bool FFileSourceControlContextMenuState::CanExecuteSCCRevert() const
{
	return bCanExecuteSCCRevert;
}

bool FFileSourceControlContextMenuState::CanExecuteSCCSync() const
{
	return bCanExecuteSCCSync;
}

bool FFileSourceControlContextMenuState::IsActionEnabled(bool bEnabled) const
{
	return bEnabled && !IsStillScanning(bEnabled);
}

bool FFileSourceControlContextMenuState::IsStillScanning(bool bEnabled) const
{
	// If we only have files selected then we only care about AsyncState
	// If at least one folder is selected then we will consider done if at least
	// one file was found that supports the given operation.
	return AsyncState != EAsyncState::Idle && (bContainsFolders ? !bEnabled : true);
}

bool FFileSourceControlContextMenuState::TrySetAsyncState(EAsyncState FromState, EAsyncState ToState)
{
	return AsyncState.compare_exchange_strong(FromState, ToState);
}

void FFileSourceControlContextMenuState::NextAsyncState()
{
	switch (AsyncState)
	{
	case EAsyncState::None:
		if (TrySetAsyncState(EAsyncState::None, EAsyncState::TryCacheCanExecute))
		{
			Async(EAsyncExecution::TaskGraphMainThread, [this, This = AsShared()]()
			{
				TryCacheCanExecuteVars(SelectedFiles, &PathsWithUnknownState);

				NextAsyncState();
			});
		}
		break;
	case EAsyncState::TryCacheCanExecute:
		if (PathsWithUnknownState.Num() == 0)
		{
			// Skip to the end since there's no more work to do
			TrySetAsyncState(EAsyncState::TryCacheCanExecute, EAsyncState::Idle);
		}
		else if (TrySetAsyncState(EAsyncState::TryCacheCanExecute, EAsyncState::SCCUpdate))
		{
			// Do an async force update to make sure to get mode up-to-date status
			FSourceControlOperationRef Operation = ISourceControlOperation::Create<FUpdateStatus>();
			SCCOperation = Operation;

			ECommandResult::Type Result = ISourceControlModule::Get().GetProvider().Execute(Operation, PathsWithUnknownState, EConcurrency::Asynchronous,
				FSourceControlOperationComplete::CreateSP(this, &FFileSourceControlContextMenuState::CacheCanExecuteVarsSCCHandler));

			if (Result != ECommandResult::Succeeded)
			{
				AsyncState = EAsyncState::Error;
			}
		}
		break;
	case EAsyncState::SCCUpdate:
		TrySetAsyncState(EAsyncState::SCCUpdate, EAsyncState::Idle);
		break;
	}
}

void FFileSourceControlContextMenuState::CacheCanExecuteVarsSCCHandler(const FSourceControlOperationRef& InOperation, ECommandResult::Type Result)
{
	EAsyncState ExpectedState = EAsyncState::SCCUpdate;
	if (Result == ECommandResult::Succeeded)
	{
		TryCacheCanExecuteVars(PathsWithUnknownState, nullptr);
	}

	NextAsyncState();
}

void FFileSourceControlContextMenuState::TryCacheCanExecuteVars(const TArray<FString>& InSelectedPaths, TArray<FString>* OptionalPathsWithUnknownState)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	const bool bUsesCheckout = SourceControlProvider.UsesCheckout();
	const bool bUsesFileRevisions = SourceControlProvider.UsesFileRevisions();

	// Check the SCC state for each package in the selected paths - this needs to happen on the main thread
	TArray<FSourceControlStateRef> SelectedStates;
	if (SourceControlProvider.GetState(InSelectedPaths, SelectedStates, EStateCacheUsage::Use) == ECommandResult::Succeeded)
	{
		for (const FSourceControlStateRef& SourceControlState : SelectedStates)
		{
			if (AsyncState == EAsyncState::Cancelled)
				return;

			if (SourceControlState->IsUnknown())
			{
				if (OptionalPathsWithUnknownState != nullptr)
				{
					OptionalPathsWithUnknownState->Add(SourceControlState->GetFilename());
				}
				continue;
			}

			bool bCanSyncCurrentItem = false;
			const bool bIsReadOnly = PlatformFile.IsReadOnly(*SourceControlState->GetFilename());

			if (bUsesFileRevisions)
			{
				if (!SourceControlState->IsCurrent())
				{
					bCanExecuteSCCSync = bCanSyncCurrentItem = true;
				}

				if (SourceControlState->CanCheckIn())
				{
					bCanExecuteSCCCheckIn = true;
				}				
			}

			if (bUsesCheckout)
			{
				if (SourceControlState->CanCheckout())
				{
					bCanExecuteSCCCheckOut = true;
				}
				else
				{
					FString CheckedOutOther;
					if (!bContainsFolders && SourceControlState->IsCheckedOutOther(&CheckedOutOther))
					{
						CheckedOutUsers.Add(FString::Printf(TEXT("%s checked out by %s"), *SourceControlState->GetFilename(), *CheckedOutOther));
					}
					else if (bCanSyncCurrentItem)
					{
						bCanExecuteSCCSyncAndCheckOut = true;
					}
				}

				if (bIsReadOnly)
				{
					bCanExecuteSCCMakeWritable = true;
				}
			}

			if (!SourceControlState->IsSourceControlled())
			{
				bCanExecuteSCCOpenForAdd = SourceControlState->CanAdd();
			}
			else
			{
				bCanExecuteSCCHistory = !SourceControlState->IsAdded();
			}

			if (SourceControlState->CanRevert())
			{
				bCanExecuteSCCRevert = true;
			}
		}
	}

	CheckedOutUsersText = FText::FromString(FString::Join(CheckedOutUsers, TEXT("\n")));
}

#undef LOCTEXT_NAMESPACE
