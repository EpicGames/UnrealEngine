// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

struct FToolMenuSection;
class FFileSourceControlContextMenuState;
class UContentBrowserDataMenuContext_FileMenu;
class UContentBrowserDataMenuContext_FolderMenu;

// Display source control options in the given menu section based on what's selected in the menu context
class FFileSourceControlContextMenu
{
public:
	~FFileSourceControlContextMenu();

	/** Makes the context menu widget */
	void MakeContextMenu(FToolMenuSection& InSection, const UContentBrowserDataMenuContext_FileMenu* InFileMenuContext, const UContentBrowserDataMenuContext_FolderMenu* InFolderMenuContext);

private:
	TSharedPtr<FFileSourceControlContextMenuState> InnerState;
};
