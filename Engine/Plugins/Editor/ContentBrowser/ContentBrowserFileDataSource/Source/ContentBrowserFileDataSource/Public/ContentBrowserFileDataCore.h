// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserItemData.h"

#define UE_API CONTENTBROWSERFILEDATASOURCE_API

struct FContentBrowserDataFilter;
class FContentBrowserFileItemDataPayload;
class FContentBrowserFileItemDataPayload_Duplication;
class FContentBrowserFolderItemDataPayload;
namespace ContentBrowserFileData { class FFileConfigData; }
namespace ContentBrowserFileData { struct FDirectoryActions; }
namespace ContentBrowserFileData { struct FFileActions; }

class FAssetThumbnail;
class UContentBrowserDataSource;

namespace ContentBrowserFileData
{

	CONTENTBROWSERFILEDATASOURCE_API FContentBrowserItemData CreateFolderItem(UContentBrowserDataSource* InOwnerDataSource, const FName InVirtualPath, const FName InInternalPath, const FString& InFilename, TWeakPtr<const ContentBrowserFileData::FDirectoryActions> InDirectoryActions);

	CONTENTBROWSERFILEDATASOURCE_API FContentBrowserItemData CreateFileItem(UContentBrowserDataSource* InOwnerDataSource, const FName InVirtualPath, const FName InInternalPath, const FString& InFilename, TWeakPtr<const ContentBrowserFileData::FFileActions> InFileActions);

	CONTENTBROWSERFILEDATASOURCE_API TSharedPtr<const FContentBrowserFolderItemDataPayload> GetFolderItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem);

	CONTENTBROWSERFILEDATASOURCE_API TSharedPtr<const FContentBrowserFileItemDataPayload> GetFileItemPayload(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem);
	
	CONTENTBROWSERFILEDATASOURCE_API void EnumerateFolderItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserFolderItemDataPayload>&)> InFolderPayloadCallback);

	CONTENTBROWSERFILEDATASOURCE_API void EnumerateFileItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserFileItemDataPayload>&)> InFilePayloadCallback);

	CONTENTBROWSERFILEDATASOURCE_API void EnumerateItemPayloads(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TFunctionRef<bool(const TSharedRef<const FContentBrowserFolderItemDataPayload>&)> InFolderPayloadCallback, TFunctionRef<bool(const TSharedRef<const FContentBrowserFileItemDataPayload>&)> InFilePayloadCallback);

	CONTENTBROWSERFILEDATASOURCE_API void SetOptionalErrorMessage(FText* OutErrorMsg, FText InErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API void MakeUniqueFilename(FString& InOutFilename);

	CONTENTBROWSERFILEDATASOURCE_API bool CanModifyDirectory(const FString& InFilename, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanModifyFile(const FString& InFilename, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanModifyItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanModifyFolderItem(const FContentBrowserFolderItemDataPayload& InFolderPayload, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanModifyFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanEditItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanEditFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool EditItems(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems);

	CONTENTBROWSERFILEDATASOURCE_API bool EditFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads);

	CONTENTBROWSERFILEDATASOURCE_API bool CanPreviewItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanPreviewFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool PreviewItems(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems);

	CONTENTBROWSERFILEDATASOURCE_API bool PreviewFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads);

	CONTENTBROWSERFILEDATASOURCE_API bool CanDuplicateItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanDuplicateFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool DuplicateItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, TSharedPtr<const FContentBrowserFileItemDataPayload_Duplication>& OutNewItemPayload);

	CONTENTBROWSERFILEDATASOURCE_API bool DuplicateFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, TSharedPtr<const FContentBrowserFileItemDataPayload_Duplication>& OutNewItemPayload);

	CONTENTBROWSERFILEDATASOURCE_API bool DuplicateItems(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, TArray<TSharedRef<const FContentBrowserFileItemDataPayload>>& OutNewFilePayloads);

	CONTENTBROWSERFILEDATASOURCE_API bool DuplicateFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads, TArray<TSharedRef<const FContentBrowserFileItemDataPayload>>& OutNewFilePayloads);

	CONTENTBROWSERFILEDATASOURCE_API bool CanDeleteItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanDeleteFolderItem(const FContentBrowserFolderItemDataPayload& InFolderPayload, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanDeleteFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool DeleteItems(const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems);

	CONTENTBROWSERFILEDATASOURCE_API bool DeleteFolderItems(TArrayView<const TSharedRef<const FContentBrowserFolderItemDataPayload>> InFolderPayloads);

	CONTENTBROWSERFILEDATASOURCE_API bool DeleteFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads);

	CONTENTBROWSERFILEDATASOURCE_API bool CanRenameItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const FString* InNewName, FText* OutErrorMsg);
	CONTENTBROWSERFILEDATASOURCE_API bool CanRenameItem(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool bCheckUniqueName, const FString* InNewName, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanRenameFolderItem(const FContentBrowserFolderItemDataPayload& InFolderPayload, const bool bCheckUniqueName, const FString* InNewName, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool CanRenameFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, const bool bCheckUniqueName, const FString* InNewName, FText* OutErrorMsg);

	CONTENTBROWSERFILEDATASOURCE_API bool RenameItem(const FFileConfigData& InConfig, const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const FString& InNewName, FName& OutNewInternalPath, FString& OutNewFilename);

	CONTENTBROWSERFILEDATASOURCE_API bool RenameFolderItem(const FFileConfigData& InConfig, const FContentBrowserFolderItemDataPayload& InFolderPayload, const FString& InNewName, FName& OutNewInternalPath, FString& OutNewFilename);

	CONTENTBROWSERFILEDATASOURCE_API bool RenameFileItem(const FContentBrowserFileItemDataPayload& InFilePayload, const FString& InNewName, FName& OutNewInternalPath, FString& OutNewFilename);

	CONTENTBROWSERFILEDATASOURCE_API bool CopyItems(const FFileConfigData& InConfig, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const FString& InDestDiskPath);

	CONTENTBROWSERFILEDATASOURCE_API bool CopyFolderItems(const FFileConfigData& InConfig, TArrayView<const TSharedRef<const FContentBrowserFolderItemDataPayload>> InFolderPayloads, const FString& InDestDiskPath);

	CONTENTBROWSERFILEDATASOURCE_API bool CopyFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads, const FString& InDestDiskPath);

	CONTENTBROWSERFILEDATASOURCE_API bool MoveItems(const FFileConfigData& InConfig, const UContentBrowserDataSource* InOwnerDataSource, TArrayView<const FContentBrowserItemData> InItems, const FString& InDestDiskPath);

	CONTENTBROWSERFILEDATASOURCE_API bool MoveFolderItems(const FFileConfigData& InConfig, TArrayView<const TSharedRef<const FContentBrowserFolderItemDataPayload>> InFolderPayloads, const FString& InDestDiskPath);

	CONTENTBROWSERFILEDATASOURCE_API bool MoveFileItems(TArrayView<const TSharedRef<const FContentBrowserFileItemDataPayload>> InFilePayloads, const FString& InDestDiskPath);

	CONTENTBROWSERFILEDATASOURCE_API bool UpdateItemThumbnail(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail);

	CONTENTBROWSERFILEDATASOURCE_API bool UpdateFileItemThumbnail(const FContentBrowserFileItemDataPayload& InFilePayload, FAssetThumbnail& InThumbnail);

	CONTENTBROWSERFILEDATASOURCE_API bool GetItemPhysicalPath(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, FString& OutDiskPath);

	CONTENTBROWSERFILEDATASOURCE_API bool GetFolderItemPhysicalPath(const FContentBrowserFolderItemDataPayload& InFolderPayload, FString& OutDiskPath);

	CONTENTBROWSERFILEDATASOURCE_API bool GetFileItemPhysicalPath(const FContentBrowserFileItemDataPayload& InFilePayload, FString& OutDiskPath);

	CONTENTBROWSERFILEDATASOURCE_API bool GetItemAttribute(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	CONTENTBROWSERFILEDATASOURCE_API bool GetFolderItemAttribute(const FContentBrowserFolderItemDataPayload& InFolderPayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	CONTENTBROWSERFILEDATASOURCE_API bool GetFileItemAttribute(const FContentBrowserFileItemDataPayload& InFilePayload, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	CONTENTBROWSERFILEDATASOURCE_API bool GetItemAttributes(const UContentBrowserDataSource* InOwnerDataSource, const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues);

	CONTENTBROWSERFILEDATASOURCE_API bool GetFolderItemAttributes(const FContentBrowserFolderItemDataPayload& InFolderPayload, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues);

	CONTENTBROWSERFILEDATASOURCE_API bool GetFileItemAttributes(const FContentBrowserFileItemDataPayload& InFilePayload, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues);

	class FDefaultFileActions
	{
	public:
		static UE_API bool ItemPassesFilter(const FName InFilePath, const FString& InFilename, const FContentBrowserDataFilter& InFilter, const bool bIsFile);
		static UE_API bool GetItemAttribute(const FName InFilePath, const FString& InFilename, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);
	};

}

#undef UE_API
