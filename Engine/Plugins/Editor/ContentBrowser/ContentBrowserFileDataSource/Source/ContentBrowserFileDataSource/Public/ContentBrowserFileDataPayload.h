// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserItemData.h"
#include "UObject/StructOnScope.h"

#define UE_API CONTENTBROWSERFILEDATASOURCE_API

struct FContentBrowserDataFilter;

namespace ContentBrowserFileData
{
	struct FCommonActions
	{
		DECLARE_DELEGATE_RetVal_ThreeParams(bool, FCanCreate, const FName /*InDestFolderPath*/, const FString& /*InDestFolder*/, FText* /*OutErrorMsg*/);
		FCanCreate CanCreate;

		/** 
		 * Queries user for arbitrary creation settings
		 * @param OutFileBasename outputs a file basename suggestion (no path, no extension) which can be empty
		 * @param OutCreationConfig outputs opaque creation settings
		 * @return false if creation gets canceled.
		 */
		DECLARE_DELEGATE_RetVal_TwoParams(bool, FConfigureCreation, FString& /*OutFileBasename*/, FStructOnScope& /*OutCreationConfig*/);
		FConfigureCreation ConfigureCreation;

		DECLARE_DELEGATE_RetVal_ThreeParams(bool, FCreate, const FName /*InFilePath*/, const FString& /*InFilename*/, const FStructOnScope& /*CreationConfig*/);
		FCreate Create;

		DECLARE_DELEGATE_RetVal_ThreeParams(bool, FCanDelete, const FName /*InFilePath*/, const FString& /*InFilename*/, FText* /*OutErrorMsg*/);
		FCanDelete CanDelete;

		DECLARE_DELEGATE_RetVal_FourParams(bool, FCanRename, const FName /*InFilePath*/, const FString& /*InFilename*/, const FString* /*InNewName*/, FText* /*OutErrorMsg*/);
		FCanRename CanRename;

		DECLARE_DELEGATE_RetVal_FourParams(bool, FCanCopy, const FName /*InFilePath*/, const FString& /*InFilename*/, const FString& /*InDestFolder*/, FText* /*OutErrorMsg*/);
		FCanCopy CanCopy;

		DECLARE_DELEGATE_RetVal_FourParams(bool, FCanMove, const FName /*InFilePath*/, const FString& /*InFilename*/, const FString& /*InDestFolder*/, FText* /*OutErrorMsg*/);
		FCanMove CanMove;

		DECLARE_DELEGATE_RetVal_FiveParams(bool, FGetAttribute, const FName /*InFilePath*/, const FString& /*InFilename*/, const bool /*InIncludeMetaData*/, const FName /*InAttributeKey*/, FContentBrowserItemDataAttributeValue& /*OutAttributeValue*/);
		FGetAttribute GetAttribute;

		DECLARE_DELEGATE_RetVal_FourParams(bool, FGetAttributes, const FName /*InFilePath*/, const FString& /*InFilename*/, const bool /*InIncludeMetaData*/, FContentBrowserItemDataAttributeValues& /*OutAttributeValues*/);
		FGetAttributes GetAttributes;

		DECLARE_DELEGATE_RetVal_ThreeParams(bool, FPassesFilter, const FName /*InFilePath*/, const FString& /*InFilename*/, const FContentBrowserDataFilter& /*InFilter*/);
		FPassesFilter PassesFilter;
	};

	struct FDirectoryActions : public FCommonActions
	{
	};

	struct FFileActions : public FCommonActions
	{
		FString TypeExtension;

		FTopLevelAssetPath TypeName;

		FText TypeDisplayName;

		FText TypeShortDescription;

		FText TypeFullDescription;

		FString DefaultNewFileName;

		FLinearColor TypeColor;

		ELaunchVerb::Type DefaultEditVerb = ELaunchVerb::Edit;

		DECLARE_DELEGATE_RetVal_ThreeParams(bool, FCanEdit, const FName /*InFilePath*/, const FString& /*InFilename*/, FText* /*OutErrorMsg*/);
		FCanEdit CanEdit;

		DECLARE_DELEGATE_RetVal_TwoParams(bool, FEdit, const FName /*InFilePath*/, const FString& /*InFilename*/);
		FEdit Edit;

		DECLARE_DELEGATE_RetVal_ThreeParams(bool, FCanPreview, const FName /*InFilePath*/, const FString& /*InFilename*/, FText* /*OutErrorMsg*/);
		FCanPreview CanPreview;

		DECLARE_DELEGATE_RetVal_TwoParams(bool, FPreview, const FName /*InFilePath*/, const FString& /*InFilename*/);
		FPreview Preview;

		DECLARE_DELEGATE_RetVal_ThreeParams(bool, FCanDuplicate, const FName /*InFilePath*/, const FString& /*InFilename*/, FText* /*OutErrorMsg*/);
		FCanDuplicate CanDuplicate;
	};

	class FFileConfigData
	{
	public:
		UE_API void SetDirectoryActions(const FDirectoryActions& InDirectoryActions);

		UE_API TSharedPtr<const FDirectoryActions> GetDirectoryActions();

		UE_API void RegisterFileActions(const FFileActions& InFileActions);

		UE_API TSharedPtr<const FFileActions> FindFileActionsForExtension(FStringView InTypeExtension) const;
		UE_API TSharedPtr<const FFileActions> FindFileActionsForFilename(FStringView InFilename) const;

		UE_API void EnumerateFileActions(TFunctionRef<bool(TSharedRef<const FFileActions>)> InCallback) const;

		UE_API FText GetDiscoveryDescription() const;

	private:
		TSharedPtr<const FDirectoryActions> DirectoryActions;

		TMap<FString, TSharedPtr<const FFileActions>> FileActionsMap;
	};
}

class FContentBrowserFolderItemDataPayload : public IContentBrowserItemDataPayload
{
public:
	explicit FContentBrowserFolderItemDataPayload(const FName InInternalPath, const FString& InFilename, TWeakPtr<const ContentBrowserFileData::FDirectoryActions> InDirectoryActions)
		: InternalPath(InInternalPath)
		, Filename(InFilename)
		, DirectoryActions(MoveTemp(InDirectoryActions))
	{
	}

	FName GetInternalPath() const
	{
		return InternalPath;
	}

	const FString& GetFilename() const
	{
		return Filename;
	}

	TSharedPtr<const ContentBrowserFileData::FDirectoryActions> GetDirectoryActions() const
	{
		return DirectoryActions.Pin();
	}

private:
	FName InternalPath;

	FString Filename;

	TWeakPtr<const ContentBrowserFileData::FDirectoryActions> DirectoryActions;
};

class FContentBrowserFileItemDataPayload : public IContentBrowserItemDataPayload
{
public:
	FContentBrowserFileItemDataPayload(const FName InInternalPath, const FString& InFilename, TWeakPtr<const ContentBrowserFileData::FFileActions> InFileActions)
		: InternalPath(InInternalPath)
		, Filename(InFilename)
		, FileActions(MoveTemp(InFileActions))
	{
	}

	FName GetInternalPath() const
	{
		return InternalPath;
	}

	const FString& GetFilename() const
	{
		return Filename;
	}

	TSharedPtr<const ContentBrowserFileData::FFileActions> GetFileActions() const
	{
		return FileActions.Pin();
	}

private:
	FName InternalPath;

	FString Filename;

	TWeakPtr<const ContentBrowserFileData::FFileActions> FileActions;
};

class FContentBrowserFileItemDataPayload_Duplication : public FContentBrowserFileItemDataPayload
{
public:
	FContentBrowserFileItemDataPayload_Duplication(const FName InInternalPath, const FString& InFilename, TWeakPtr<const ContentBrowserFileData::FFileActions> InFileActions, const FString& InSourceFilename)
		: FContentBrowserFileItemDataPayload(InInternalPath, InFilename, MoveTemp(InFileActions))
		, SourceFilename(InSourceFilename)
	{
	}

	const FString& GetSourceFilename() const
	{
		return SourceFilename;
	}

private:
	/** The source file that we're duplicating */
	FString SourceFilename;
};


class FContentBrowserFileItemDataPayload_Creation : public FContentBrowserFileItemDataPayload
{
public:
	FContentBrowserFileItemDataPayload_Creation(const FName InInternalPath, const FString& InFilename, TWeakPtr<const ContentBrowserFileData::FFileActions> InFileActions, FStructOnScope&& InCreationConfig)
		: FContentBrowserFileItemDataPayload(InInternalPath, InFilename, MoveTemp(InFileActions))
		, CreationConfig(MoveTemp(InCreationConfig))
	{
	}

	const FStructOnScope& GetCreationConfig() const
	{
		return CreationConfig;
	}

private:
	FStructOnScope CreationConfig;
};

#undef UE_API
