// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserFileDataPayload.h"
#include "ContentBrowserFileDataSource.generated.h"

#define UE_API CONTENTBROWSERFILEDATASOURCE_API

class UToolMenu;
class FContentBrowserFileDataDiscovery;
class IAssetTypeActions;
struct FFileChangeData;

USTRUCT()
struct FContentBrowserCompiledFileDataFilter
{
	GENERATED_BODY()

public:
	FName VirtualPathName;
	FString VirtualPath;
	bool bRecursivePaths = false;
	EContentBrowserItemAttributeFilter ItemAttributeFilter = EContentBrowserItemAttributeFilter::IncludeNone;
	TSharedPtr<FPathPermissionList> PermissionList;
	TArray<FString> FileExtensionsToInclude;
};

UCLASS(MinimalAPI)
class UContentBrowserFileDataSource : public UContentBrowserDataSource
{
	GENERATED_BODY()

public:
	UE_API void Initialize(const ContentBrowserFileData::FFileConfigData& InConfig, const bool InAutoRegister = true);

	UE_API virtual void Shutdown() override;

	UE_API void AddFileMount(const FName InFileMountPath, const FString& InFileMountDiskPath);

	UE_API void RemoveFileMount(const FName InFileMountPath);

	UE_API bool HasFileMount(const FName InFileMountPath) const;

	UE_API virtual void Tick(const float InDeltaTime);

	UE_API virtual void CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter) override;

	UE_API virtual void EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual void EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual void EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) override;

	UE_API virtual bool CanCreateFolder(const FName InPath, FText* OutErrorMsg) override;

	UE_API virtual bool CreateFolder(const FName InPath, const TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>& HideFolderIfEmptyFilter, FContentBrowserItemDataTemporaryContext& OutPendingItem) override;

	UE_API virtual bool IsDiscoveringItems(FText* OutStatus = nullptr) override;

	UE_API virtual bool PrioritizeSearchPath(const FName InPath) override;

	UE_API virtual bool IsFolderVisible(const FName InPath, const EContentBrowserIsFolderVisibleFlags InFlags, const FContentBrowserFolderContentsFilter& InContentsFilter) override;

	UE_API virtual bool DoesItemPassFilter(const FContentBrowserItemData& InItem, const FContentBrowserDataCompiledFilter& InFilter) override;

	UE_API virtual bool GetItemAttribute(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue) override;

	UE_API virtual bool GetItemAttributes(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues) override;

	UE_API virtual bool GetItemPhysicalPath(const FContentBrowserItemData& InItem, FString& OutDiskPath) override;

	UE_API virtual bool CanEditItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;

	UE_API virtual bool EditItem(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool BulkEditItems(TArrayView<const FContentBrowserItemData> InItems) override;

	UE_API virtual bool CanPreviewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;

	UE_API virtual bool PreviewItem(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool BulkPreviewItems(TArrayView<const FContentBrowserItemData> InItems) override;

	UE_API virtual bool CanDuplicateItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;

	UE_API virtual bool DuplicateItem(const FContentBrowserItemData& InItem, FContentBrowserItemDataTemporaryContext& OutPendingItem) override;

	UE_API virtual bool BulkDuplicateItems(TArrayView<const FContentBrowserItemData> InItems, TArray<FContentBrowserItemData>& OutNewItems) override;

	UE_API virtual bool CanDeleteItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg) override;

	UE_API virtual bool DeleteItem(const FContentBrowserItemData& InItem) override;

	UE_API virtual bool BulkDeleteItems(TArrayView<const FContentBrowserItemData> InItems) override;

	UE_API virtual bool CanRenameItem(const FContentBrowserItemData& InItem, const FString* InNewName, const IContentBrowserHideFolderIfEmptyFilter* HideFolderIfEmptyFilter, FText* OutErrorMsg) override;

	UE_API virtual bool RenameItem(const FContentBrowserItemData& InItem, const FString& InNewName, FContentBrowserItemData& OutNewItem) override;

	UE_API virtual bool CanCopyItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg) override;

	UE_API virtual bool CopyItem(const FContentBrowserItemData& InItem, const FName InDestPath) override;

	UE_API virtual bool BulkCopyItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath) override;

	UE_API virtual bool CanMoveItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg) override;

	UE_API virtual bool MoveItem(const FContentBrowserItemData& InItem, const FName InDestPath) override;

	UE_API virtual bool BulkMoveItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath) override;

	UE_API virtual bool UpdateThumbnail(const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail) override;

	UE_API virtual void BuildRootPathVirtualTree() override;

protected:

	struct FFileMount
	{
		FString DiskPath;
		FDelegateHandle DirectoryWatcherHandle;
	};

	struct FDiscoveredItem
	{
		enum class EType : uint8
		{
			Directory,
			File,
		};

		EType Type;
		FString DiskPath;
		TSet<FName> ChildItems;
	};

	UE_API FContentBrowserItemData CreateFolderItem(const FName InInternalPath, const FString& InFilename);

	UE_API FContentBrowserItemData CreateFileItem(const FName InInternalPath, const FString& InFilename);

	UE_API FContentBrowserItemData CreateItemFromDiscovered(const FName InInternalPath, const FDiscoveredItem& InDiscoveredItem);

	UE_API TSharedPtr<const FContentBrowserFolderItemDataPayload> GetFolderItemPayload(const FContentBrowserItemData& InItem) const;

	UE_API TSharedPtr<const FContentBrowserFileItemDataPayload> GetFileItemPayload(const FContentBrowserItemData& InItem) const;

	UE_API bool HideFolderIfEmpty(const IContentBrowserHideFolderIfEmptyFilter& HideFolderIfEmptyFilter, FName Path, FStringView PathString) const;

	UE_API bool IsKnownFileMount(const FName InMountPath, FString* OutDiskPath = nullptr) const;

	UE_API bool IsRootFileMount(const FName InMountPath, FString* OutDiskPath = nullptr) const;

	UE_API void AddDiscoveredItem(FDiscoveredItem::EType InItemType, const FString& InMountPath, const FString& InDiskPath, const bool bIsRootPath);
	UE_API void AddDiscoveredItemImpl(FDiscoveredItem::EType InItemType, const FString& InMountPath, const FString& InDiskPath, const FName InChildMountPathName, const bool bIsRootPath);

	UE_API void RemoveDiscoveredItem(const FString& InMountPath);
	UE_API void RemoveDiscoveredItem(const FName InMountPath);
	UE_API void RemoveDiscoveredItemImpl(const FName InMountPath, const bool bParentIsOrphan);

	UE_API void OnPathPopulated(const FName InPath);
	UE_API void OnPathPopulated(const FStringView InPath);

	UE_API void OnAlwaysShowPath(const FName InPath);
	UE_API void OnAlwaysShowPath(const FStringView InPath);

	/** Called when a file in a watched directory changes on disk */
	UE_API void OnDirectoryChanged(const TArray<FFileChangeData>& InFileChanges, const FString InFileMountPath, const FString InFileMountDiskPath);

	UE_API void PopulateAddNewContextMenu(UToolMenu* InMenu);

	UE_API void OnNewFileRequested(const FName InDestFolderPath, const FString InDestFolder, TSharedRef<const ContentBrowserFileData::FFileActions> InFileActions, UContentBrowserDataMenuContext_AddNewMenu::FOnBeginItemCreation InOnBeginItemCreation);

	UE_API bool OnValidateItemName(const FContentBrowserItemData& InItem, const FString& InProposedName, FText* OutErrorMsg);

	UE_API FContentBrowserItemData OnFinalizeCreateFolder(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg);

	UE_API FContentBrowserItemData OnFinalizeCreateFile(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg);

	UE_API FContentBrowserItemData OnFinalizeDuplicateFile(const FContentBrowserItemData& InItemData, const FString& InProposedName, FText* OutErrorMsg);

	UE_API bool PassesFilters(const FName InPath, const int32 InFolderDepthChecked, const FContentBrowserCompiledFileDataFilter& InFileDataFilter) const;
	static UE_API bool PassesFilters(const FStringView InPath, const FDiscoveredItem& InDiscoveredItem, const int32 InFolderDepthChecked, const FContentBrowserCompiledFileDataFilter& InFileDataFilter);

	ContentBrowserFileData::FFileConfigData Config;

	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;

	TSortedMap<FName, FFileMount, FDefaultAllocator, FNameFastLess> RegisteredFileMounts;
	TMap<FName, TArray<FName>> RegisteredFileMountRoots;

	TMap<FName, FDiscoveredItem> DiscoveredItems;

	TSharedPtr<FContentBrowserFileDataDiscovery> BackgroundDiscovery;

	/**
	 * The set of folders that should always be visible, even if they contain no files in the Content Browser view.
	 * This will include root file mounts, and any folders that have been created directly (or indirectly) by a user action.
	 */
	TSet<FString> AlwaysVisibleFolders;

	/**
	 * A cache of folders that contain no files in the Content Browser view.
	 */
	TSet<FString> EmptyFolders;
};

#undef UE_API
